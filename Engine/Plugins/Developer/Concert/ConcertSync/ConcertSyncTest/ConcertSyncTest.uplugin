{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Concert Sync - Test",
	"Description": "Plugin to enables multi-users tests",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [],
	"Modules": [
		{
			"Name": "ConcertSyncTest",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault",
			"ProgramAllowList": []
		}
	],
	"Plugins": [
		{
			"Name": "ConcertClientSharedSlate",
			"Enabled": true
		},
		{
			"Name": "ConcertSharedSlate",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncServer",
			"Enabled": true
		}
	]
}
