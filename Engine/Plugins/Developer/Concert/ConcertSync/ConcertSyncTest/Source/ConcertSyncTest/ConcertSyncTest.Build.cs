// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ConcertSyncTest : ModuleRules
	{
		public ConcertSyncTest(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Concert",
					"ConcertClientSharedSlate",
					"ConcertSharedSlate",
					"ConcertTransport",
					"ConcertSyncCore",
					"ConcertSyncClient",
					"ConcertSyncServer",
					"Engine"
				}
			);

			PrivateIncludePaths.AddRange(
				new string[]
				{
					Path.Combine(GetModuleDirectory("Core"), "Tests"),
				}
			);
		}
	}
}
