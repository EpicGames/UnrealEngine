// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertDataStoreTests.h"
#include "ConcertMessages.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "IConcertClientDataStore.h"
#include "ConcertDataStore.h"
#include "ConcertClientLocalDataStore.h"
#include "IConcertSession.h"
#include "Scratchpad/ConcertScratchpad.h"
#include "Util/Mock/MockUtils.h"

// Defines a namespace to test FText.
#define LOCTEXT_NAMESPACE "ConcertDataStoreTests" 

/** Flags used for the Concert data store tests. */
static const EAutomationTestFlags ConcertDataStoreTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter; // | EAutomationTestFlags::SmokeFilter;

namespace UE::ConcertSyncTests::DatastoreTests
{
// This function is implemented in ConcertSyncServer -> ConcertServerDataStore.cpp and prevent exposing ConcertServerDataStore.h publicly
// as this is not required for general purpose. For the test, we pass a mocked session in which the data store server hook itself to send/
// receive events/requests/responses. This enable testing the client/server data store integration by just mocking the transport layer
// between them
extern TSharedPtr<void> MakeConcerteServerDataStoreForTest(TSharedRef<IConcertServerSession> Session, bool bIsContentReplicationEnabled);

// Those functions are implemented in ConcertSyncClient -> ConcertClientDataStore.cpp and prevent exposing ConcertClientDataStore.h publicly
// as this is not required for general purpose. This function returns an instance than can be passed back to the 2 others external functions.
extern TSharedRef<IConcertClientDataStore> MakeConcertClientDataStoreForTest(TSharedRef<IConcertClientSession> Session);
extern int32 GetConcertClientDataStoreCacheSize(const IConcertClientDataStore& ClientStore);
extern FConcertDataStoreValueConstPtr GetConcertClientDataStoreCachedValue(const IConcertClientDataStore& ClientStore, const FName& Key, const FName& TypeName);
extern uint32 GetCompareExchangePayloadOptimizationThreshold();

template<typename T>
FConcertDataStoreValueConstPtr GetClientCachedValue(const IConcertClientDataStore& ClientStore, const FName& Key)
{
	return GetConcertClientDataStoreCachedValue(ClientStore, Key, TConcertDataStoreType<T>::GetFName());
}

uint32 GetClientCacheSize(const IConcertClientDataStore& ClientStore)
{
	return GetConcertClientDataStoreCacheSize(ClientStore);
}

FString GetTestSessionRootPath()
{
	return FPaths::ProjectIntermediateDir() / TEXT("ConcertDataStoreTest");
}

bool operator==(const FText& lhs, const FText& rhs)
{
	// This is not probably not how FText should be compared, but for the purpose of our
	// tests, this is good enough because we don't really test localization.
	return lhs.ToString() == rhs.ToString();
}

bool operator!=(const FText& lhs, const FText& rhs)
{
	// This is not probably not how FText should be compared, but for the purpose of our
	// tests, this is good enough because we don't really test localization.
	return lhs.ToString() != rhs.ToString();
}

/** Implements a not-working IConcertServerSession. It must be further overridden to implement just what is required by the tests */
class FConcertServerSessionBaseMock : public IConcertServerSession
{
public:
	FConcertServerSessionBaseMock() 
		: Id(FGuid::NewGuid())
		, Name("FConcertServerSessionBaseMock")
	{ }

	// IConcertSession Begin.
	virtual const FGuid& GetId() const override																								{ return Id; }
	virtual const FString& GetName() const override																							{ return NotMocked<const FString&>(Name); }
	virtual void SetName(const FString&) override																							{ return NotMocked<void>(); }
	virtual FMessageAddress GetClientAddress(const FGuid& ClientEndpointId) const override													{ return NotMocked<FMessageAddress>(); }
	virtual const FConcertSessionInfo& GetSessionInfo() const override																		{ return NotMocked<const FConcertSessionInfo&>(SessionInfo); }
	virtual FString GetSessionWorkingDirectory() const override																				{ return GetTestSessionRootPath() / Id.ToString(); }
	virtual TArray<FGuid> GetSessionClientEndpointIds() const override																		{ return NotMocked<TArray<FGuid>>(); }
	virtual TArray<FConcertSessionClientInfo> GetSessionClients() const override															{ return NotMocked<TArray<FConcertSessionClientInfo>>(); }
	virtual bool FindSessionClient(const FGuid&, FConcertSessionClientInfo&) const override													{ return NotMocked<bool>(); }
	virtual void Startup() override																											{ return NotMocked<void>(); }
	virtual void Shutdown() override																										{ return NotMocked<void>(); };
	virtual FConcertScratchpadRef GetScratchpad() const override																			{ return NotMocked<FConcertScratchpadRef>(); }
	virtual FConcertScratchpadPtr GetClientScratchpad(const FGuid&) const override															{ return NotMocked<FConcertScratchpadRef>(); }
	virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName&, const TSharedRef<IConcertSessionCustomEventHandler>&) override { return NotMocked<FDelegateHandle>(); }
	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override			{ return NotMocked<void>(); }
	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override						{ return NotMocked<void>(); }
	virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override													{ return NotMocked<void>(); }
	virtual void InternalSendCustomEvent(const UScriptStruct*, const void*, const TArray<FGuid>&, EConcertMessageFlags, TOptional<FConcertSequencedCustomEvent> InSequenceId={}) override			{ return NotMocked<void>(); }
	virtual void InternalRegisterCustomRequestHandler(const FName&, const TSharedRef<IConcertSessionCustomRequestHandler>&) override		{ return NotMocked<void>(); }
	virtual void InternalUnregisterCustomRequestHandler(const FName&) override																{ return NotMocked<void>(); }
	virtual void InternalSendCustomRequest(const UScriptStruct*, const void*, const FGuid&, const TSharedRef<IConcertSessionCustomResponseHandler>&) override { NotMocked<void>(); }
	// IConcertSession End.

	// IConcertServerSession Begin
	virtual FOnConcertServerSessionTick& OnTick() override                          { return NotMocked<FOnConcertServerSessionTick&>(Tick); }
	virtual FOnConcertServerSessionClientChanged& OnSessionClientChanged() override { return NotMocked<FOnConcertServerSessionClientChanged&>(ConnectionChanged); }
	virtual FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& OnConcertMessageAcknowledgementReceived() override { return NotMocked<FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint&>(AckReceived); }
	// IConcertServerSession End

protected:
	FGuid Id;

	// Those below are unused mocked data member, but required to get the code compiling.
	FString Name;
	FConcertSessionInfo SessionInfo;
	FOnConcertServerSessionTick Tick;
	FOnConcertServerSessionClientChanged ConnectionChanged;
	FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint AckReceived;
};

/** Implements a not-working IConcertClientSession. It must be further overridden to implement just what is required by the tests */
class FConcertClientSessionBaseMock : public IConcertClientSession
{
public:
	FConcertClientSessionBaseMock() 
		: Id(FGuid::NewGuid()) 
		, Name("FConcertClientSessionBaseMock") 
	{ }

	// IConcertSession Begin.
	virtual const FGuid& GetId() const override																								{ return Id; }
	virtual const FString& GetName() const override																							{ return NotMocked<const FString&>(Name); }
	virtual const FConcertSessionInfo& GetSessionInfo() const override																		{ return NotMocked<const FConcertSessionInfo&>(SessionInfo); }
	virtual FString GetSessionWorkingDirectory() const override																				{ return GetTestSessionRootPath() / Id.ToString(); }
	virtual TArray<FGuid> GetSessionClientEndpointIds() const override																		{ return NotMocked<TArray<FGuid>>(); }
	virtual TArray<FConcertSessionClientInfo> GetSessionClients() const override															{ return NotMocked<TArray<FConcertSessionClientInfo>>(); }
	virtual bool FindSessionClient(const FGuid&, FConcertSessionClientInfo&) const override													{ return NotMocked<bool>(); }
	virtual void Startup() override																											{ return NotMocked<void>(); }
	virtual void Shutdown() override																										{ return NotMocked<void>(); };
	virtual FConcertScratchpadRef GetScratchpad() const override																			{ return NotMocked<FConcertScratchpadRef>(); }
	virtual FConcertScratchpadPtr GetClientScratchpad(const FGuid&) const override															{ return NotMocked<FConcertScratchpadRef>(); }
	virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName&, const TSharedRef<IConcertSessionCustomEventHandler>&) override { return NotMocked<FDelegateHandle>(); }
	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override			{ return NotMocked<void>(); }
	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override						{ return NotMocked<void>(); }
	virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override													{ return NotMocked<void>(); }
	virtual void InternalSendCustomEvent(const UScriptStruct*, const void*, const TArray<FGuid>&, EConcertMessageFlags, TOptional<FConcertSequencedCustomEvent> InSequenceId={}) override			{ return NotMocked<void>(); }
	virtual void InternalRegisterCustomRequestHandler(const FName&, const TSharedRef<IConcertSessionCustomRequestHandler>&) override		{ return NotMocked<void>(); }
	virtual void InternalUnregisterCustomRequestHandler(const FName&) override																{ return NotMocked<void>(); }
	virtual void InternalSendCustomRequest(const UScriptStruct*, const void*, const FGuid&, const TSharedRef<IConcertSessionCustomResponseHandler>&) override { NotMocked<void>(); }
	// IConcertSession End.

	virtual FConcertSequencedCustomEventManager& GetSequencedEventManager() override
	{
		return NotMocked<FConcertSequencedCustomEventManager&>(CustomEventSequenceManager);
	}
	// IConcertClientSession Begin
	virtual EConcertConnectionStatus GetConnectionStatus() const override            { return NotMocked<EConcertConnectionStatus>(EConcertConnectionStatus::Connected); }
	virtual FGuid GetSessionClientEndpointId() const override                        { return NotMocked<FGuid>(); }
	virtual FGuid GetSessionServerEndpointId() const override                        { return NotMocked<FGuid>(); }
	virtual const FConcertClientInfo& GetLocalClientInfo() const override            { return NotMocked<const FConcertClientInfo&>(ClientInfo); }
	virtual void UpdateLocalClientInfo(const FConcertClientInfoUpdate&) override     { return NotMocked<void>(); }
	virtual void Connect() override                                                  { return NotMocked<void>(); }
	virtual void Disconnect() override                                               { return NotMocked<void>(); }

	virtual EConcertSendReceiveState GetSendReceiveState() const override
	{
		return NotMocked<EConcertSendReceiveState>(EConcertSendReceiveState::Default);
	}

	virtual void SetSendReceiveState(EConcertSendReceiveState InSendReceiveState) override
	{
		return NotMocked<void>();
	}

	virtual FOnConcertClientSessionTick& OnTick() override                           { return NotMocked<FOnConcertClientSessionTick&>(Tick); }
	virtual FOnConcertClientSessionConnectionChanged& OnConnectionChanged() override { return NotMocked<FOnConcertClientSessionConnectionChanged&>(ConnectionChanged); }
	virtual FOnConcertClientSessionClientChanged& OnSessionClientChanged() override  { return NotMocked<FOnConcertClientSessionClientChanged&>(ClientChanged); }
	virtual FOnConcertSessionRenamed& OnSessionRenamed() override                    { return NotMocked<FOnConcertSessionRenamed&>(SessionRenamed); }
	// IConcertClientSession End

	virtual void HandleCustomEvent(const UScriptStruct* EventType, const void* EventData) = 0;

protected:
	FGuid Id;

	// Those below are unused mocked data member, but required to get the code compiling.
	FString Name;
	FConcertSessionInfo SessionInfo;
	FOnConcertClientSessionTick Tick;
	FOnConcertClientSessionConnectionChanged ConnectionChanged;
	FOnConcertClientSessionClientChanged ClientChanged;
	FOnConcertSessionRenamed SessionRenamed;
	FConcertSequencedCustomEventManager CustomEventSequenceManager;
	FConcertClientInfo ClientInfo;
};

/** Specializes the base concert server session to act as a fake server session. */
class FConcertServerSessionMock : public FConcertServerSessionBaseMock
{
public:
	virtual void InternalSendCustomEvent(const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& TargetEndpointIds, EConcertMessageFlags, TOptional<FConcertSequencedCustomEvent> InSequenceId={}) override
	{
		for (const FGuid& TargetEndPointId : TargetEndpointIds)
		{
			int i = 0;
			for (const FGuid& ClientEndPointId : ClientEndpoints)
			{
				if (TargetEndPointId == ClientEndPointId)
				{
					// Dispatch the event on the client immediately.
					ClientSessions[i]->HandleCustomEvent(EventType, EventData);
				}
				++i;
			}
		}
	}

	virtual void InternalRegisterCustomRequestHandler(const FName& RequestMessageType, const TSharedRef<IConcertSessionCustomRequestHandler>& Handler) override
	{
		CustomRequestHandlers.Add(RequestMessageType, Handler);
	}

	virtual void InternalUnregisterCustomRequestHandler(const FName& RequestMessageType) override
	{
		CustomRequestHandlers.Remove(RequestMessageType);
	}

	virtual TArray<FGuid> GetSessionClientEndpointIds() const override
	{
		return ClientEndpoints;
	}

	// Called by the tests to connect a client to the server.
	void ConnectClient(const FGuid& ClientEndpointId, FConcertClientSessionBaseMock& ClientSession)
	{
		// Notify the server that a new client connected. The server data store will replicate its content on the client.
		FConcertSessionClientInfo Info;
		Info.ClientEndpointId = ClientEndpointId;
		ClientEndpoints.Add(ClientEndpointId);
		ClientSessions.Add(&ClientSession);
		ConnectionChanged.Broadcast(*this, EConcertClientStatus::Connected, Info);
	}

	// Called by the FConcertClientSessionMock to dispatch a request
	void DispatchRequest(const FGuid& RequesterEndpointId, const UScriptStruct* RequestType, const void* RequestData, const TSharedRef<IConcertSessionCustomResponseHandler>& ResponseHandler)
	{
		if (TSharedPtr<IConcertSessionCustomRequestHandler>* RequestHandler = CustomRequestHandlers.Find(RequestType->GetFName()))
		{
			// Set up who's sending the request.
			FConcertSessionContext Context;
			Context.SourceEndpointId = RequesterEndpointId;

			// Dispatch the request
			FStructOnScope ResponsePayload((*RequestHandler)->GetResponseType());
			EConcertSessionResponseCode Result = (*RequestHandler)->HandleRequest(Context, RequestData, ResponsePayload.GetStructMemory());
			if (Result == EConcertSessionResponseCode::Success || Result == EConcertSessionResponseCode::Failed)
			{
				// Dispatch the response.
				ResponseHandler->HandleResponse(ResponsePayload.GetStructMemory());
			}
			else
			{
				check(false); // The test suite is not expected to fire any other result than Success or Failed.
			}
		}
	}

	virtual FOnConcertServerSessionClientChanged& OnSessionClientChanged() override
	{
		return ConnectionChanged;
	}

private:
	/** Map of session custom request handlers */
	TMap<FName, TSharedPtr<IConcertSessionCustomRequestHandler>> CustomRequestHandlers;

	/** Connected client endpoints. */
	TArray<FGuid> ClientEndpoints;

	/** Connected clients sessions. */
	TArray<FConcertClientSessionBaseMock*> ClientSessions;
};

/** Specializes the base concert client session to act as a fake client session. */
class FConcertClientSessionMock : public FConcertClientSessionBaseMock
{
public:
	FConcertClientSessionMock(const FGuid& ClientEndpointId, FConcertServerSessionMock& Server)
		: ServerMock(Server)
		, EndpointId(ClientEndpointId)
	{
	}

	virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName& EventMessageType, const TSharedRef<IConcertSessionCustomEventHandler>& Handler) override
	{
		CustomEventHandlers.FindOrAdd(EventMessageType).Add(Handler);
		return Handler->GetHandle();
	}

	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override
	{
		TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventMessageType);
		if (HandlerArrayPtr)
		{
			for (auto It = HandlerArrayPtr->CreateIterator(); It; ++It)
			{
				if ((*It)->GetHandle() == EventHandle)
				{
					It.RemoveCurrent();
					break;
				}
			}
		}
	}

	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override
	{
		TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventMessageType);
		if (HandlerArrayPtr)
		{
			HandlerArrayPtr->RemoveAllSwap([EventHandler](const TSharedPtr<IConcertSessionCustomEventHandler>& InHandler) -> bool
			{
				return InHandler->HasSameObject(EventHandler);
			});
		}
	}

	virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override
	{
		CustomEventHandlers.Remove(EventMessageType);
	}

	virtual void InternalSendCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& DestinationEndpointId, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler) override
	{
		// Directly dispatch to the server session.
		ServerMock.DispatchRequest(EndpointId, RequestType, RequestData, Handler);
	}

	virtual void HandleCustomEvent(const UScriptStruct* EventType, const void* EventData) override
	{
		TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventType->GetFName());
		if (HandlerArrayPtr)
		{
			FConcertSessionContext DummyContext;
			for (const TSharedPtr<IConcertSessionCustomEventHandler>& Handler : *HandlerArrayPtr)
			{
				Handler->HandleEvent(DummyContext, EventData);
			}
		}
	}

	virtual FGuid GetSessionClientEndpointId() const override
	{
		return EndpointId;
	}

	virtual FGuid GetSessionServerEndpointId() const override
	{
		return FGuid(0, 0, 0, 0);
	}

private:
	FConcertServerSessionMock& ServerMock;
	FGuid EndpointId;
	TMap<FName, TArray<TSharedPtr<IConcertSessionCustomEventHandler>>> CustomEventHandlers;
};

/** Base class to perform Concert data store client/server tests. */
class FConcertDataStoreClientServerTest : public FAutomationTestBase
{
public:
	struct FClientInfo
	{
		FClientInfo(const FGuid& ClientEndPointId, FConcertServerSessionMock& Server)
			: ClientSessionMock(MakeShared<FConcertClientSessionMock>(ClientEndPointId, Server))
			, ClientDataStore(MakeConcertClientDataStoreForTest(ClientSessionMock))
		{
		}

		TSharedRef<FConcertClientSessionBaseMock> ClientSessionMock;
		TSharedRef<IConcertClientDataStore> ClientDataStore;
	};

	FConcertDataStoreClientServerTest(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

	IConcertClientDataStore& ConnectClient()
	{
		FGuid ClientEndpointId(0, 0, 0, Clients.Num() + 1); // {0, 0, 0, 0} is used by the server.

		Clients.Add(MakeUnique<FClientInfo>(ClientEndpointId, *ServerSessionMock));
		ServerSessionMock->ConnectClient(ClientEndpointId, *(Clients.Last()->ClientSessionMock));
		return *Clients.Last()->ClientDataStore;
	}

	void InitServer(bool bEnableContentReplication = false)
	{
		// Reset everything to be able to rerun the tests. The test framework doesn't destruct/reconstruct this object
		// at every run, so just ensure we start with a clean state.
		Clients.Empty();
		ServerDataStore.Reset();
		ServerSessionMock = MakeShared<FConcertServerSessionMock>();
		ServerDataStore = MakeConcerteServerDataStoreForTest(ServerSessionMock.ToSharedRef(), bEnableContentReplication);
	}

	/** Ensures a functor with this signature is not called. */
	template<typename T>
	struct EnsureNotCalled
	{
		// Just ensure the function is not called.
		void operator()(const FName&, TOptional<T>) { check(false); };
	};

	/** Ensures a key/value pair was added. */
	template<typename T>
	void EnsureValueAdded(const TConcertDataStoreResult<T>& Result, const T& ExpectedValue)
	{
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Result.GetValue() == ExpectedValue);
		TestTrueExpr(Result.IsValid());
	}

	/** Ensures the expected value was fetched. */
	template<typename T>
	void EnsureValueFetched(const TConcertDataStoreResult<T>& Result, const T& ExpectedValue)
	{
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Fetched);
		TestTrueExpr(Result.GetValue() == ExpectedValue);
		TestTrueExpr(Result.IsValid());
	}

	/** Ensures the desired value was exchanged. */
	template<typename T>
	void EnsureValueExchanged(const TConcertDataStoreResult<T>& Result, const T& ExpectedValue)
	{
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Exchanged);
		TestTrueExpr(Result.GetValue() == ExpectedValue);
		TestTrueExpr(Result.IsValid());
	}

	template<typename T>
	void EnsureTypeMismatch(const TConcertDataStoreResult<T>& Result)
	{
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::TypeMismatch);
		TestTrueExpr(!Result.IsValid()); // Server doesn't send back any value on error.
		TestTrueExpr(!Result); // Server doesn't send back any value on error.
	}

	template<typename T>
	void EnsureNotFound(const TConcertDataStoreResult<T>& Result)
	{
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::NotFound);
		TestTrueExpr(!Result.IsValid()); // Server doesn't send back any value on error.
		TestTrueExpr(!Result); // Server doesn't send back any value on error.
	}

	template<typename T>
	void TestCommonOperations(IConcertClientDataStore& Store, const FName& KeyName, T StoreValue, T ExchangeValue, T UnexpectedValue)
	{
		// The test expects the value to be different.
		//TestTrueExpr(StoreValue != ExchangeValue);

		// Ensure a new key is added.
		EnsureValueAdded(Store.FetchOrAdd(KeyName, StoreValue).Get(), StoreValue);

		// Ensure the stored key value is properly fetched.
		EnsureValueFetched(Store.FetchAs<T>(KeyName).Get(), StoreValue);

		// Ensure the existing key is not added, but fetched with the proper value and version.
		EnsureValueFetched(Store.FetchOrAdd(KeyName, ExchangeValue).Get(), StoreValue);

		// Ensure the previous operation did not overwrite the key value.
		EnsureValueFetched(Store.FetchAs<T>(KeyName).Get(), StoreValue);

		// Ensure the stored key value is exchanged to the desired value.
		EnsureValueExchanged(Store.CompareExchange(KeyName, StoreValue, ExchangeValue).Get(), ExchangeValue);

		// Ensure the previous exchanged value was correctly stored.
		EnsureValueFetched(Store.FetchAs<T>(KeyName).Get(), ExchangeValue);

		// Ensure that exchange fails if the expected value doesn't match the stored value. It should return the existing value instead.
		EnsureValueFetched(Store.CompareExchange(KeyName, UnexpectedValue, StoreValue).Get(), ExchangeValue);

		// Ensure the previous exchange operation did not overwrite the existing value.
		EnsureValueFetched(Store.FetchAs<T>(KeyName).Get(), ExchangeValue);
	}

	template<typename T, typename U>
	void TestTypeMismatch(IConcertClientDataStore& Store1, IConcertClientDataStore& Store2, const FName& KeyName, T StoreValue, U MismatchTypeValue)
	{
		// Add a value. The Store1 will have it cached locally.
		EnsureValueAdded(Store1.FetchOrAdd(KeyName, StoreValue).Get(), StoreValue);

		// Ensure the value cannot be read as another type. (Local client check using the cache, no server call)
		EnsureTypeMismatch(Store1.FetchAs<U>(KeyName).Get());

		// The key already exist, ensure it cannot be read/overwritten as another type. (Local client check using the cache, no server call)
		EnsureTypeMismatch(Store1.FetchOrAdd(KeyName, MismatchTypeValue).Get());

		// Ensure the value cannot be exchanged as another type. (Local client check using the cache, no server call)
		EnsureTypeMismatch(Store1.CompareExchange(KeyName, MismatchTypeValue, MismatchTypeValue).Get());

		// Ensure the key value still hold its initial value.
		EnsureValueFetched(Store1.FetchAs<T>(KeyName).Get(), StoreValue);

		// Ensure the value cannot be added as another type by another client. (Server check, this client hasn't the value cached, the server push notifications are off)
		EnsureTypeMismatch(Store2.FetchOrAdd(KeyName, MismatchTypeValue).Get());
	}

private:
	TSharedPtr<FConcertServerSessionMock> ServerSessionMock;
	TSharedPtr<void> ServerDataStore;
	TArray<TUniquePtr<FClientInfo>> Clients;
};

} // namespace UE::ConcertSyncTests

/** Ensures the Concert data store correctly versions the stored values. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertDataStoreValueVersioning, "VirtualProduction.Concert.DataStore.ValueVersioning", ConcertDataStoreTestFlags)

bool FConcertDataStoreValueVersioning::RunTest(const FString& Parameters)
{
	// Wraps a value into its corresonding USTRUCT()
	int32 Value = 10;
	const typename TConcertDataStoreType<decltype(Value)>::StructType& StructWrappedValue = TConcertDataStoreType<decltype(Value)>::AsStructType(Value);
	FConcertSessionSerializedPayload SerializedValue;
	SerializedValue.SetPayload(TConcertDataStoreType<decltype(Value)>::StructType::StaticStruct(), &StructWrappedValue);

	// FConcertDataStore::FetchOrAdd() automatically set version 1 when added.
	{
		FConcertDataStore Store;
		FName Key(TEXT("Key1"));
		TestTrueExpr(Store.FetchOrAdd(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue).Code == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == 1);
	}

	// FConcertDataStore::Store() automatically set version 1 if not specified.
	{
		FConcertDataStore Store;
		FName Key(TEXT("Key1"));
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue).Code == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == 1);
	}

	// FConcertDataStore::Store() used the specified version.
	{
		FConcertDataStore Store;
		FName Key(TEXT("Key1"));
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue, TOptional<uint32>(55)).Code == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == 55);
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue, TOptional<uint32>(75)).Code == EConcertDataStoreResultCode::Exchanged);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == 75);
	}

	// FConcertDataStore::Store() automatically increment version on update if version is not specified.
	{
		FConcertDataStore Store;
		FName Key(TEXT("Key1"));
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue, TOptional<uint32>(55)).Code == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == 55);
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue).Code == EConcertDataStoreResultCode::Exchanged);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == 56);
	}

	// FConcertDataStore::Store() automatically wraps around version number in case of overflow.
	{
		FConcertDataStore Store;
		FName Key(TEXT("Key1"));
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue, TOptional<uint32>(TNumericLimits<uint32>::Max())).Code == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == TNumericLimits<uint32>::Max());
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue).Code == EConcertDataStoreResultCode::Exchanged);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == 0);
	}

	// FConcertDataStore::Fetch()/FetchOrAdd() returns the correct version number.
	{
		FConcertDataStore Store;
		FName Key(TEXT("Key1"));
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue, TOptional<uint32>(32)).Code == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Store.Fetch(Key, TConcertDataStoreType<decltype(Value)>::GetFName()).Value->Version == 32);
		TestTrueExpr(Store.FetchOrAdd(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue).Value->Version == 32);
		TestTrueExpr(Store.GetVersion(Key).GetValue() == 32);
	}

	// Multi-versions
	{
		FConcertDataStore Store(FConcertDataStore::EUpdatePolicy::Replace);
		FName Key(TEXT("Key1"));
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue, TOptional<uint32>(32)).Code == EConcertDataStoreResultCode::Added);
		FConcertDataStoreResult Result1 = Store.FetchOrAdd(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue);
		TestTrueExpr(Store.Store(Key, TConcertDataStoreType<decltype(Value)>::GetFName(), SerializedValue, TOptional<uint32>(42)).Code == EConcertDataStoreResultCode::Exchanged);
		FConcertDataStoreResult Result2 = Store.Fetch(Key, TConcertDataStoreType<decltype(Value)>::GetFName());

		// The result values/version are expected to be immutable.
		TestTrueExpr(Result1.Code == EConcertDataStoreResultCode::Fetched && Result1.Value->Version == 32);
		TestTrueExpr(Result2.Code == EConcertDataStoreResultCode::Fetched && Result2.Value->Version == 42);
	}

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data store correctly handles the common operations. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerCommonOperations, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.CommonOperations", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerCommonOperations::RunTest(const FString& Parameters)
{
	InitServer();
	IConcertClientDataStore& Client = ConnectClient();
	TestCommonOperations(Client, FName(TEXT("Key_i8")), int8(33), int8(-20), int8(77));
	TestCommonOperations(Client, FName(TEXT("Key_u8")), uint8(10), uint8(80), uint8(0));
	TestCommonOperations(Client, FName(TEXT("Key_i16")), int16(33), int16(-20), int16(77));
	TestCommonOperations(Client, FName(TEXT("Key_u16")), uint16(10), uint16(80), uint16(0));
	TestCommonOperations(Client, FName(TEXT("Key_i32")), int32(33), int32(-20), int32(77));
	TestCommonOperations(Client, FName(TEXT("Key_u32")), uint32(10), uint32(80), uint32(0));
	TestCommonOperations(Client, FName(TEXT("Key_i64")), 10ll, -80ll, 0ll);
	TestCommonOperations(Client, FName(TEXT("Key_u64")), 10ull, 80ull, 0ull);
	TestCommonOperations(Client, FName(TEXT("Key_flt")), 10.0f, 80.0f, 0.0f);
	TestCommonOperations(Client, FName(TEXT("Key_dbl")), 10.0, 80.0, 0.0);
	TestCommonOperations(Client, FName(TEXT("Key_bool")), true, false, true);
	TestCommonOperations(Client, FName(TEXT("Key_FName")), FName(TEXT("foo")), FName(TEXT("bar")), FName(TEXT("Hello")));
	TestCommonOperations(Client, FName(TEXT("Key_FStr")), FString(TEXT("foo")), FString(TEXT("bar")), FString(TEXT("Hello")));
	TestCommonOperations(Client, FName(TEXT("Key_FText")), FText(LOCTEXT("FooKey", "FooText")), FText(LOCTEXT("BarKey", "BarText")), FText(LOCTEXT("HelloKey", "HelloText")));
	TestCommonOperations(Client, FName(TEXT("Key_Custom")), FConcertDataStore_CustomTypeTest{1, 2, 0.5f, {1}}, FConcertDataStore_CustomTypeTest{127, 8, 2.5f, {1}}, FConcertDataStore_CustomTypeTest{0, 0, 0.0f, {1}});

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data store correctly handles the "key not found" cases. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerKeyNotFound, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.KeyNotFound", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerKeyNotFound::RunTest(const FString& Parameters)
{
	InitServer();
	IConcertClientDataStore& Client = ConnectClient();

	FName Key(TEXT("JaneDoe"));
	EnsureNotFound(Client.FetchAs<int64>(Key).Get());
	EnsureNotFound(Client.CompareExchange(Key, 10ull, 1ull).Get());
	EnsureNotFound(Client.CompareExchange(Key, 10.0, 1.0).Get());

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data store correctly handles the "type mismatch" cases. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerTypeMismatch, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.TypeMismatch", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerTypeMismatch::RunTest(const FString& Parameters)
{
	InitServer();
	IConcertClientDataStore& Client1 = ConnectClient();
	IConcertClientDataStore& Client2 = ConnectClient();

	// Ensure we detect type mismatch.
	TestTypeMismatch(Client1, Client2, FName(TEXT("TypeMismatch_i64_float")), 10ll, 0.5f);
	TestTypeMismatch(Client1, Client2, FName(TEXT("TypeMismatch_i64_u64")),   10ll, 1ull);
	TestTypeMismatch(Client1, Client2, FName(TEXT("TypeMismatch_i64_bool")),  10ll, true);

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data store optimize the "compare and exchange" operation, to avoid sending the payload when using the version is more optimal. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerCompareExchangeOptimization, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.CompareExchangeOptimization", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerCompareExchangeOptimization::RunTest(const FString& Parameters)
{
	InitServer();
	IConcertClientDataStore& Client = ConnectClient();

	FName KeyName(TEXT("CompareExchangeUsesVersion"));
	FConcertDataStore_CustomTypeTest StoreValue{0, 0, 0.0f, {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20}};
	FConcertDataStore_CustomTypeTest ExchangeValue{0, 0, 0.0f, {0}};

	// Just ensure the payload is large enough to enable the optimization.
	TestTrueExpr(StoreValue.IntArray.Num() * sizeof(decltype(StoreValue.IntArray)::ElementType) > UE::ConcertSyncTests::DatastoreTests::GetCompareExchangePayloadOptimizationThreshold());

	// Add a new key. The client is expected to cache the stored value at version 1.
	EnsureValueAdded(Client.FetchOrAdd(KeyName, StoreValue).Get(), StoreValue);

	// Exchange the value. Since the client has the value cached at version 1 and its payload is quite large, the client should only send the version.
	EnsureValueExchanged(Client.CompareExchange(KeyName, StoreValue, ExchangeValue).Get(), ExchangeValue);

	// Ensure the previous exchanged value was correctly stored.
	EnsureValueFetched(Client.FetchAs<FConcertDataStore_CustomTypeTest>(KeyName).Get(), ExchangeValue);

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data store client correctly caches the key/values when it receives the response to its requests form the server. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerClientCache, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.ClientCache", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerClientCache::RunTest(const FString& Parameters)
{
	// NOTE: The server doesn't push update(s) to the client(s) in this test because we want to ensure
	//       the client use its local cache as expected.
	InitServer();
	IConcertClientDataStore& Client1 = ConnectClient();
	IConcertClientDataStore& Client2 = ConnectClient();

	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 0);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client2) == 0);
	
	FName Key("Key");
	int32 Value = 100;
	uint32 Version = 1;

	// Ensure "not found" errors do not affect the cache.
	EnsureNotFound(Client1.FetchAs<int64>(Key).Get());
	EnsureNotFound(Client1.CompareExchange<int64>(Key, 0, 0).Get());
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 0);

	// Ensure successfully adding a value populates the client local cache.
	EnsureValueAdded(Client1.FetchOrAdd(Key, Value).Get(), Value);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 1);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->Version == Version);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->DeserializeUnchecked<int32>() == Value);
	TestTrueExpr(Client1.FetchAs<int32>(Key).Get().GetValue() == Value); // Should read from the cache.
	TestTrueExpr(Client1.FetchOrAdd<int32>(Key, Value * 2).Get().GetValue() == Value); // Should read from the cache.

	// Ensure "type mismatch" errors do not affect the cache.
	EnsureTypeMismatch(Client1.FetchOrAdd<float>(Key, 0.0f).Get());
	EnsureTypeMismatch(Client1.CompareExchange<float>(Key, 0.0f, 1.0f).Get());
	EnsureTypeMismatch(Client1.FetchAs<float>(Key).Get());
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 1);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->Version == Version);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->DeserializeUnchecked<int32>() == Value);

	// Ensure successfully exchanging a value updates the cache.
	EnsureValueExchanged(Client1.CompareExchange(Key, Value, Value + 1).Get(), Value + 1);
	++Value;
	++Version;
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 1);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->Version == Version);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->DeserializeUnchecked<int32>() == Value);
	TestTrueExpr(Client1.FetchAs<int32>(Key).Get().GetValue() == Value); // Should read from the cache.
	TestTrueExpr(Client1.FetchOrAdd<int32>(Key, Value * 2).Get().GetValue() == Value); // Should read from the cache.

	// Ensure failing to exchange a value does not affect the cache. (Should be local failure)
	EnsureValueFetched(Client1.CompareExchange(Key, Value + 44, Value + 88).Get(), Value);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 1);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->Version == Version);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->DeserializeUnchecked<int32>() == Value);
	TestTrueExpr(Client1.FetchAs<int32>(Key).Get().GetValue() == Value); // Should read from the cache.
	TestTrueExpr(Client1.FetchOrAdd<int32>(Key, Value * 2).Get().GetValue() == Value); // Should read from the cache.

	// Client 2 should not have anything cached because server push notification are off. So operations relying on its local cache should return "not found".
	EnsureNotFound(Client2.FetchAs<int32>(Key).Get());
	EnsureNotFound(Client2.CompareExchange<int32>(Key, 0, 1).Get());

	// Force client 2 to cache key 1 by calling FetchOrAdd().
	EnsureValueFetched(Client2.FetchOrAdd(Key, Value).Get(), Value);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client2) == 1);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client2, Key)->Version == Version);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client2, Key)->DeserializeUnchecked<int32>() == Value);
	TestTrueExpr(Client2.FetchAs<int32>(Key).Get().GetValue() == Value); // Should read from its cache.
	TestTrueExpr(Client2.FetchOrAdd<int32>(Key, Value * 2).Get().GetValue() == Value); // Should read from its cache.

	// Client 2 updates the key value.
	EnsureValueExchanged(Client2.CompareExchange(Key, Value, Value + 1).Get(), Value + 1);

	// Ensure client 1 fails to exchange the value because its value is now outdated per client 2 update. Ensure its cache gets updated from the response.
	EnsureValueFetched(Client1.CompareExchange(Key, Value, Value + 10).Get(), Value + 1);
	++Value; // This is the value as exchanged by client 2 a couple of lines above.
	++Version;
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 1);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->Version == Version);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->DeserializeUnchecked<int32>() == Value);
	TestTrueExpr(Client1.FetchAs<int32>(Key).Get().GetValue() == Value); // Should read from its cache.
	TestTrueExpr(Client1.FetchOrAdd<int32>(Key, Value * 2).Get().GetValue() == Value); // Should read from its cache.

	// Client 2 will update the key multiple times and put it back to the value cached by client 1.
	EnsureValueExchanged(Client2.CompareExchange(Key, Value, Value + 1).Get(), Value + 1);
	++Value;
	++Version;
	EnsureValueExchanged(Client2.CompareExchange(Key, Value, Value - 1).Get(), Value - 1);
	--Value;
	++Version;

	// Ensure client 1 compare exchange successfully, and cache the updated value with the latest version.
	EnsureValueExchanged(Client1.CompareExchange<int32>(Key, Value, Value + 50).Get(), Value + 50);
	Value += 50;
	++Version;
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 1);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->Version == Version); // Version 4.
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<int32>(Client1, Key)->DeserializeUnchecked<int32>() == Value);

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data storeserver correctly push notifications to client and client populate its cache. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerChangeNotification, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.ChangeNotification", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerChangeNotification::RunTest(const FString& Parameters)
{
	bool bEnableServerPushNotification = true;
	InitServer(bEnableServerPushNotification);
	IConcertClientDataStore& Client1 = ConnectClient();
	IConcertClientDataStore& Client2 = ConnectClient();

	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 0);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client2) == 0);

	using T = int32;

	// Client 1 adds a value to the store. Server will push a notification, observed by client 2.
	FName Key1(TEXT("Key1"));
	T Value = 44;
	TestTrueExpr(Client1.FetchOrAdd(Key1, Value).Get().GetCode() == EConcertDataStoreResultCode::Added);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client1) == 1);

	// Client 2 must have the key1 cached by now.
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client2) == 1);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<T>(Client2, Key1)->DeserializeUnchecked<T>() == Value);

	// Client 2 updates the key, server will push a notification to client 1.
	TestTrueExpr(Client2.CompareExchange(Key1, Value, Value + 1).Get().GetCode() == EConcertDataStoreResultCode::Exchanged);
	++Value; // The value was exchanged, set its new value.

	// Client 1 cache should be updated by now.
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<T>(Client1, Key1)->DeserializeUnchecked<T>() == Value);

	// Add few other key/value.
	FName Key2(TEXT("Key2"));
	TestTrueExpr(Client2.FetchOrAdd(Key2, Value).Get().GetCode() == EConcertDataStoreResultCode::Added);
	FName Key3(TEXT("Key3"));
	TestTrueExpr(Client1.FetchOrAdd(Key3, Value).Get().GetCode() == EConcertDataStoreResultCode::Added);
	FName Key4(TEXT("Key4"));
	TestTrueExpr(Client1.FetchOrAdd(Key4, Value).Get().GetCode() == EConcertDataStoreResultCode::Added);

	// Connect a third client. Ensure its cache gets populated by the server.
	IConcertClientDataStore& Client3 = ConnectClient();
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCacheSize(Client3) == 4);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<T>(Client3, Key1)->DeserializeUnchecked<T>() == Value);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<T>(Client3, Key2)->DeserializeUnchecked<T>() == Value);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<T>(Client3, Key3)->DeserializeUnchecked<T>() == Value);
	TestTrueExpr(UE::ConcertSyncTests::DatastoreTests::GetClientCachedValue<T>(Client3, Key4)->DeserializeUnchecked<T>() == Value);

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data store client correctly call the change handler. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerChangeNotificationHandler, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.ChangeNotificationHandler", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerChangeNotificationHandler::RunTest(const FString& Parameters)
{
	bool bEnableServerPushNotification = true;
	InitServer(bEnableServerPushNotification);
	IConcertClientDataStore& Client1 = ConnectClient();
	IConcertClientDataStore& Client2 = ConnectClient();

	FName IntKey(TEXT("IntKey"));
	int32 IntValue = 99;

	FName CustomKey(TEXT("CustomKey"));
	FConcertDataStore_CustomTypeTest CustomValue1{1, 2, 0.5f, {1}};
	FConcertDataStore_CustomTypeTest CustomValue2{1, 2, 0.5f, {1}};

	FName FloatKey(TEXT("FloatKey"));
	float FloatValue = 9.0f;

	int32 Client2NotificationCount = 0;
	Client2.RegisterChangeNotificationHandler<int32>(IntKey, [&](const FName& InKey, TOptional<int32> InValue)
	{
		TestTrueExpr(InKey == IntKey);
		TestTrueExpr(InValue.IsSet());
		TestTrueExpr(InValue.GetValue() == IntValue);
		Client2NotificationCount++;
	});

	Client2.RegisterChangeNotificationHandler<FConcertDataStore_CustomTypeTest>(CustomKey, [&](const FName& InKey, TOptional<FConcertDataStore_CustomTypeTest> InValue)
	{
		TestTrueExpr(InKey == CustomKey);
		TestTrueExpr(InValue.IsSet());
		FConcertDataStore_CustomTypeTest Copy = MoveTemp(InValue.GetValue()); // This is how clients are expected to keep a copy of large types.

		if (Client2NotificationCount == 0)
		{
			TestTrueExpr(Copy == CustomValue1);
		}
		else
		{
			TestTrueExpr(Copy == CustomValue2);
		}
		Client2NotificationCount++;
	});

	// Type mismatch. FloatKey is expected to be a float, but we register using a int64.
	Client2.RegisterChangeNotificationHandler<int64>(FloatKey, [&](const FName& InKey, TOptional<int64> InValue)
	{
		TestTrueExpr(InKey == FloatKey);
		TestTrueExpr(!InValue.IsSet()); // In case of type mismatch, the value is not set.
		Client2NotificationCount++;
	});

	// Ensure client 1 is not called back as it is the one performing all the changes.
	Client1.RegisterChangeNotificationHandler<int32>(IntKey, EnsureNotCalled<int32>());
	Client1.RegisterChangeNotificationHandler<float>(FloatKey, EnsureNotCalled<float>());
	Client1.RegisterChangeNotificationHandler<FConcertDataStore_CustomTypeTest>(CustomKey, EnsureNotCalled<FConcertDataStore_CustomTypeTest>());

	// Add IntKey to client 1, it should trigger a notification in client 2.
	EnsureValueAdded(Client1.FetchOrAdd(IntKey, IntValue).Get(), IntValue);
	TestTrueExpr(Client2NotificationCount == 1);

	// Add CustomKey to client 1, it should trigger a notification in client 2.
	EnsureValueAdded(Client1.FetchOrAdd(CustomKey, CustomValue1).Get(), CustomValue1);
	TestTrueExpr(Client2NotificationCount == 2);

	// Add FloatKey to client 1, it should trigger a notification in client 2.
	EnsureValueAdded(Client1.FetchOrAdd(FloatKey, FloatValue).Get(), FloatValue);
	TestTrueExpr(Client2NotificationCount == 3);

	// Connects a 3rd client, the server should populate its cache.
	IConcertClientDataStore& Client3 = ConnectClient();

	// Ensure the key value is pushed on connection (Default option)
	int32 Client3NotificationCount = 0;
	Client3.RegisterChangeNotificationHandler<int32>(IntKey, [&](const FName& InKey, TOptional<int32> InValue)
	{
		TestTrueExpr(InKey == IntKey);
		TestTrueExpr(InValue.IsSet());
		TestTrueExpr(InValue.GetValue() == IntValue);
		Client3NotificationCount++;
	});
	TestTrueExpr(Client3NotificationCount == 1);

	// Ensure the client is not called on initial value if NotifyOnInitialValue is not set.
	Client3.RegisterChangeNotificationHandler<float>(FloatKey, EnsureNotCalled<float>(), EConcertDataStoreChangeNotificationOptions::None);

	// Ensure the client is not called on initial value if types do not match and NotifyOnTypeMismatch is not set.
	Client3.RegisterChangeNotificationHandler<int64>(CustomKey, EnsureNotCalled<int64>(), EConcertDataStoreChangeNotificationOptions::NotifyOnInitialValue);

	// Ensure the Client3 is not called by Client1 update when CustomKey change because the Client3 did not set option NotifyOnTypeMismatch and register a callback of int64 rather than FConcertDataStore_CustomTypeTest.
	EnsureValueExchanged(Client1.CompareExchange(CustomKey, CustomValue1, CustomValue2).Get(), CustomValue2);

	// Ensure unregister works.
	FName DoubleKey("DoubleKey");
	int DoubleKeyNotificationCount = 0;
	Client1.RegisterChangeNotificationHandler<double>(DoubleKey, [&](const FName& InKey, TOptional<double> InValue)
	{
		DoubleKeyNotificationCount++;
	});

	// Client 2 adds a new key, this should trigger an update in client 1 handler.
	double DoubleValue = 0.0;
	EnsureValueAdded(Client2.FetchOrAdd(DoubleKey, DoubleValue).Get(), DoubleValue);
	TestTrueExpr(DoubleKeyNotificationCount == 1);

	// Client 1 unregister its change notification handler on double key.
	Client1.UnregisterChangeNotificationHander(DoubleKey);

	// Client 2 update double key. This should not trigger a callback in client 1 handler.
	EnsureValueExchanged(Client2.CompareExchange(DoubleKey, DoubleValue, DoubleValue + 2).Get(), DoubleValue + 2);
	TestTrueExpr(DoubleKeyNotificationCount == 1);

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data store blocking API works. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerBlockingApi, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.BlockingAPI", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerBlockingApi::RunTest(const FString& Parameters)
{
	InitServer();
	IConcertClientDataStore& Client1 = ConnectClient();
	IConcertClientDataStore& Client2 = ConnectClient();

	// Implement a small scenario.
	auto ScenerioFunc = [this](IConcertClientDataStore& Store)
	{
		// Initialize a key named "CameraId" which will be incremented every time a new camera is spanned.
		FName CameraIdKey(TEXT("CameraId"));
		int64 CurrCameraId = 0;

		// If the operation succeeded.
		if (TConcertDataStoreResult<int64> InitCameraIdResult = Store.FetchOrAdd(CameraIdKey, CurrCameraId).Get())
		{
			CurrCameraId = InitCameraIdResult.GetValue();
		}

		// Try securing a new camera id, assuming the current value is the last one read.
		int64 NextCameraId = CurrCameraId + 1;
		TConcertDataStoreResult<int64> NextCameraIdResult = Store.CompareExchange(CameraIdKey, CurrCameraId, NextCameraId).Get();

		// As long as we fail to exchange the value, read the fetched value, increment it and try again.
		while (NextCameraIdResult.GetCode() != EConcertDataStoreResultCode::Exchanged)
		{
			// We expect that if we did not exchange the key, it was fetched.
			TestTrueExpr(NextCameraIdResult.GetCode() == EConcertDataStoreResultCode::Fetched);

			// Read the value that was stored when the compare/exchange hit the backend.
			CurrCameraId = NextCameraIdResult.GetValue();
			NextCameraId = CurrCameraId + 1;

			// Try exchanging again, expecting that the last read value is the one stored. Set the desired value as the current + 1.
			NextCameraIdResult = Store.CompareExchange(CameraIdKey, CurrCameraId, NextCameraId).Get();
		}

		// The 'NextCameraId' value was exchanged and stored, make it our current known value and use it as our unique camera id.
		CurrCameraId = NextCameraId;

		// Spawn a new camera, generating its name using the camera id.
		// SpawnCamera(CurrCameraId);
	};

	// Run the scenario for 2 clients because the scenario should exercise two different execution paths.
	ScenerioFunc(Client1);
	ScenerioFunc(Client2);

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
}

/** Ensures the Concert data store using continuation API works. */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientServerContinuationApi, UE::ConcertSyncTests::DatastoreTests::FConcertDataStoreClientServerTest, "VirtualProduction.Concert.DataStore.ClientServer.ContinuationAPI", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientServerContinuationApi::RunTest(const FString& Parameters)
{
	InitServer();
	IConcertClientDataStore& Client = ConnectClient();

	FName Key(TEXT("CameraId")); // The shared variable name.
	int64 Value = 0; // The initial value if not existing yet.
	int64 CameraId;
	bool bCameraIdAcquired = false;
	bool bNewIdGenerated = false;

	// Try to fetch the specified key value (a basic type), it the key doesn't exist, add it with the specified value.
	Client.FetchOrAdd(Key, Value).Next([this, Key, Value, &CameraId, &bCameraIdAcquired](const TConcertDataStoreResult<int64>& Result)
	{
		// If the key was added or fetched.
		if (Result)
		{
			CameraId = Result.GetValue();
			bCameraIdAcquired = true;
		}
		else
		{
			// The key already existed, but the value was not a int64.
			TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::TypeMismatch);
		}
	});

	// The test runs synchronously. So we expect this to be true here.
	TestTrueExpr(bCameraIdAcquired);

	while (!bNewIdGenerated)
	{
		Client.CompareExchange(Key, CameraId, CameraId + 1).Next([&CameraId, &bNewIdGenerated](const TConcertDataStoreResult<int64>& Result)
		{
			if (Result.GetCode() == EConcertDataStoreResultCode::Fetched)
			{
				// Another client changed the value before us, update the expected and try again.
				CameraId = Result.GetValue();
			}
			else if (Result.GetCode() == EConcertDataStoreResultCode::Exchanged)
			{
				// We exchanged the value. The value we stored is CameraId + 1.
				CameraId = CameraId + 1;
				bNewIdGenerated = true;
			}
		});
	}

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	// The test runs synchronously. So we expect this to be true here.
	TestTrueExpr(bNewIdGenerated);
	return true;
}

/** Ensures the Concert data store local client store works. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertDataStoreClientPrivateStore, "VirtualProduction.Concert.DataStore.ClientPrivateStore", ConcertDataStoreTestFlags)

bool FConcertDataStoreClientPrivateStore::RunTest(const FString& Parameters)
{
	FConcertClientLocalDataStore DataStore;

	// FetchOrAdd()
	{
		auto Result = DataStore.FetchOrAdd(FName(TEXT("MyKey1")), 100ull);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Result.GetValue() == 100ull);
	}

	{
		auto Result = DataStore.FetchOrAdd(FName(TEXT("MyKey1")), 0ull);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Fetched);
		TestTrueExpr(Result.GetValue() == 100ull);
	}

	{
		auto Result = DataStore.FetchOrAdd(FName(TEXT("MyKey1")), 0.0f);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::TypeMismatch);
		TestTrueExpr(!Result.IsValid());
	}

	// Store()
	{
		auto Result = DataStore.Store(FName(TEXT("MyKey2")), 10ull);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Added);
		TestTrueExpr(Result.GetValue() == 10ull);
	}

	{
		auto Result = DataStore.Store(FName(TEXT("MyKey2")), 20ull);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Exchanged);
		TestTrueExpr(Result.GetValue() == 20ull);
	}

	{
		auto Result = DataStore.Store(FName(TEXT("MyKey2")), 2.0f);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::TypeMismatch);
		TestTrueExpr(!Result.IsValid());
	}

	// FetchAs()
	{
		auto Result = DataStore.FetchAs<uint64>(FName(TEXT("MyKey2")));
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Fetched);
		TestTrueExpr(Result.GetValue() == 20ull);
	}

	{
		auto Result = DataStore.FetchAs<uint64>(FName(TEXT("NotFoundKey")));
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::NotFound);
		TestTrueExpr(!Result.IsValid());
	}

	{
		auto Result = DataStore.FetchAs<float>(FName(TEXT("MyKey2")));
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::TypeMismatch);
		TestTrueExpr(!Result.IsValid());
	}

	// CompareExchange()
	{
		auto Result = DataStore.CompareExchange(FName(TEXT("MyKey2")), 20ull, 30ull);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Exchanged);
		TestTrueExpr(Result.GetValue() == 30ull);
	}

	{
		auto Result = DataStore.CompareExchange(FName(TEXT("MyKey2")), 20ull, 30ull);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::Fetched);
		TestTrueExpr(Result.GetValue() == 30ull);
	}

	{
		auto Result = DataStore.CompareExchange(FName(TEXT("NotFoundKey")), true, false);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::NotFound);
		TestTrueExpr(!Result.IsValid());
	}

	{
		auto Result = DataStore.CompareExchange(FName(TEXT("MyKey2")), 30.0f, 20.0f);
		TestTrueExpr(Result.GetCode() == EConcertDataStoreResultCode::TypeMismatch);
		TestTrueExpr(!Result.IsValid());
	}

	// bool operator.
	{
		FName MyKey(TEXT("MyKey3"));
		uint64 MyValue = 100ull;
		if (auto Stored = DataStore.FetchOrAdd(MyKey, MyValue))
		{
			if ((Stored = DataStore.Store(MyKey, Stored.GetValue() + 10)))
			{
				TestTrueExpr(DataStore.FetchAs<uint64>(MyKey).GetValue() == MyValue + 10);
				if ((Stored = DataStore.CompareExchange(MyKey, MyValue + 10, MyValue + 20)))
				{
					TestTrueExpr(DataStore.FetchAs<uint64>(MyKey).GetValue() == MyValue + 20);
				}
			}
		}
		TestTrueExpr(DataStore.FetchAs<uint64>(MyKey).GetValue() == MyValue + 20);
		TestTrueExpr(!DataStore.CompareExchange(MyKey, 0.0f, 3.0f)); // Type mismatch.
	}

	// Multi-versions.
	{
		FName MyKey(TEXT("MyKey4"));
		uint64 MyValue = 100ull;
		auto Result1 = DataStore.FetchOrAdd(MyKey, MyValue);
		auto Result2 = DataStore.Store(MyKey, MyValue + 10);
		auto Result3 = DataStore.CompareExchange(MyKey, MyValue + 10, MyValue + 20);
		TestTrueExpr(Result1.GetValue() == MyValue);
		TestTrueExpr(Result2.GetValue() == MyValue + 10);
		TestTrueExpr(Result3.GetValue() == MyValue + 20);
	}

	IFileManager::Get().DeleteDirectory(*UE::ConcertSyncTests::DatastoreTests::GetTestSessionRootPath(), false, true);

	return true;
};

#undef LOCTEXT_NAMESPACE
