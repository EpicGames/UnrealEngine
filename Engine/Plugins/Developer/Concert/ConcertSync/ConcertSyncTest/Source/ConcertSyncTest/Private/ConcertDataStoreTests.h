// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertDataStoreTests.generated.h"

/** A custom type for ConcertDatastore testing purpose. */
USTRUCT()
struct FConcertDataStore_CustomTypeTest
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	int8 Int8Value = 0;

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	int64 Int64Value = 0;

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	float FloatValue = 0;

	UPROPERTY(VisibleAnywhere, Category = "Concert Message")
	TArray<int> IntArray;

	bool operator==(const FConcertDataStore_CustomTypeTest& other) const
	{
		return (Int8Value == other.Int8Value) && (Int64Value == other.Int64Value) && (FloatValue == other.FloatValue) && (IntArray == other.IntArray);
	}

	bool operator!=(const FConcertDataStore_CustomTypeTest& other) const
	{
		return !operator==(other);
	}
};
