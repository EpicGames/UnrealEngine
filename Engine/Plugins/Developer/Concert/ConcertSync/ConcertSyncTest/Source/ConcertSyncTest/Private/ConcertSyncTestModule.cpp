// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

/**
 * Empty Module
 */
class FConcertSyncTestModule : public IModuleInterface {};

IMPLEMENT_MODULE(FConcertSyncTestModule, ConcertSyncTest);

