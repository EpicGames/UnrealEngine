// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Messages/ReplicationActivity.h"

#include "Components/StaticMeshComponent.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertSessionDatabaseTest, "VirtualProduction.Concert.Database.SessionDatabase", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FConcertSessionDatabaseTest::RunTest(const FString& Parameters)
{
	const FString TestSessionPath_Server = FPaths::ProjectIntermediateDir() / TEXT("ConcertDatabaseTest_Server");
	const FString TestSessionPath_Client = FPaths::ProjectIntermediateDir() / TEXT("ConcertDatabaseTest_Client");

	FConcertSyncSessionDatabase SessionDatabase_Server;
	FConcertSyncSessionDatabase SessionDatabase_Client;

	if (!SessionDatabase_Server.Open(TestSessionPath_Server))
	{
		AddError(TEXT("Failed to open server database"));
	}

	if (!SessionDatabase_Client.Open(TestSessionPath_Client))
	{
		AddError(TEXT("Failed to open client database"));
	}

	// Populate the server database with various activities
	if (SessionDatabase_Server.IsValid())
	{
		// Add the endpoint the activity will be associated with
		const FGuid EndpointId = FGuid::NewGuid();
		{
			FConcertSyncEndpointData EndpointData;
			EndpointData.ClientInfo.Initialize();
			if (!SessionDatabase_Server.SetEndpoint(EndpointId, EndpointData))
			{
				AddError(FString::Printf(TEXT("Failed to set endpoint on server database: %s"), *SessionDatabase_Server.GetLastError()));
			}
		}

		// Add some activity that would have been generated by a client
		{
			int64 ActivityId = 0;
			int64 EventId = 0;

			FConcertSyncConnectionActivity ConnectionActivity;
			ConnectionActivity.EndpointId = EndpointId;
			ConnectionActivity.EventData.ConnectionEventType = EConcertSyncConnectionEventType::Connected;
			if (!SessionDatabase_Server.AddConnectionActivity(ConnectionActivity, ActivityId, EventId))
			{
				AddError(FString::Printf(TEXT("Failed to add connection activity to server database: %s"), *SessionDatabase_Server.GetLastError()));
			}

			FConcertSyncLockActivity LockActivity;
			LockActivity.EndpointId = EndpointId;
			LockActivity.EventData.ResourceNames.Add(TEXT("/Game/TestAsset"));
			LockActivity.EventData.LockEventType = EConcertSyncLockEventType::Locked;
			if (!SessionDatabase_Server.AddLockActivity(LockActivity, ActivityId, EventId))
			{
				AddError(FString::Printf(TEXT("Failed to add lock activity to server database: %s"), *SessionDatabase_Server.GetLastError()));
			}
			LockActivity.EventData.LockEventType = EConcertSyncLockEventType::Unlocked;
			if (!SessionDatabase_Server.AddLockActivity(LockActivity, ActivityId, EventId))
			{
				AddError(FString::Printf(TEXT("Failed to add lock activity to server database: %s"), *SessionDatabase_Server.GetLastError()));
			}

			FConcertSyncTransactionActivity TransactionActivity;
			TransactionActivity.EndpointId = EndpointId;
			TransactionActivity.EventData.Transaction.ModifiedPackages.Add(TEXT("/Game/TestAsset"));
			if (!SessionDatabase_Server.AddTransactionActivity(TransactionActivity, ActivityId, EventId))
			{
				AddError(FString::Printf(TEXT("Failed to add transaction activity to server database: %s"), *SessionDatabase_Server.GetLastError()));
			}

			FConcertSyncActivity PackageActivityBasePart;
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageActivityBasePart.EndpointId = EndpointId;
			PackageInfo.PackageName = TEXT("/Game/TestAsset");
			SessionDatabase_Server.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			if (!SessionDatabase_Server.AddPackageActivity(PackageActivityBasePart, PackageInfo, PackageDataStream, ActivityId, EventId))
			{
				AddError(FString::Printf(TEXT("Failed to add package activity to server database: %s"), *SessionDatabase_Server.GetLastError()));
			}

			const FGuid StreamId = FGuid::NewGuid();
			const FSoftObjectPath StaticMeshComponent(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0"));
			FConcertReplicationStream Stream;
			Stream.BaseDescription.Identifier = StreamId;
			FConcertReplicatedObjectInfo& ObjectInfo = Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(StaticMeshComponent);
			ObjectInfo.ClassPath = UStaticMeshComponent::StaticClass();
			const TOptional<FConcertPropertyChain> Property_RelativeX = FConcertPropertyChain::CreateFromPath(*UStaticMeshComponent::StaticClass(), { TEXT("RelativeLocation"), TEXT("X") });
			if (ensure(Property_RelativeX))
			{
				ObjectInfo.PropertySelection.ReplicatedProperties.Add(*Property_RelativeX);
			}
			FConcertSyncReplicationPayload_LeaveReplication LeaveReplication;
			LeaveReplication.Streams.Add(Stream);
			LeaveReplication.OwnedObjects.Add({ StreamId, StaticMeshComponent });
			FConcertSyncReplicationActivity StreamActivity;
			StreamActivity.EventData.SetPayload(LeaveReplication);
			if (!SessionDatabase_Server.AddReplicationActivity(StreamActivity, ActivityId, EventId))
			{
				AddError(FString::Printf(TEXT("Failed to add replication stream activity to server database: %s"), *SessionDatabase_Server.GetLastError()));
			}
		}

		// Close and re-open the client database to verify that it isn't corrupted
		if (SessionDatabase_Client.IsValid())
		{
			if (!SessionDatabase_Client.Close())
			{
				AddError(TEXT("Failed to close client database"));
			}
			if (!SessionDatabase_Client.Open(TestSessionPath_Client))
			{
				AddError(TEXT("Failed to open client database"));
			}
		}
	}

	// Populate the client database from the server data
	if (SessionDatabase_Client.IsValid())
	{
		// Endpoint IDs must be synced before activity, as the database enforces that the activity was from a known endpoint
		SessionDatabase_Server.EnumerateEndpoints([this, &SessionDatabase_Client](FConcertSyncEndpointIdAndData&& InEndpointData)
		{
			if (!SessionDatabase_Client.SetEndpoint(InEndpointData.EndpointId, InEndpointData.EndpointData))
			{
				AddError(FString::Printf(TEXT("Failed to set endpoint '%s' on client database: %s"), *InEndpointData.EndpointId.ToString(), *SessionDatabase_Client.GetLastError()));
				return false;
			}
			return true;
		});

		// This loop represents both what a sync command queue would do when queuing events on the server, and what the client would do when receiving them
		// This is a very simple example that just syncs all data for every activity, but a more complex case would be skipping the event data for non-live transactions and non-head package revisions
		SessionDatabase_Server.EnumerateActivityIdsAndEventTypes([this, &SessionDatabase_Server, &SessionDatabase_Client](const int64 InActivityId, const EConcertSyncActivityEventType InEventType)
		{
			switch (InEventType)
			{
			case EConcertSyncActivityEventType::Connection:
			{
				FConcertSyncConnectionActivity ConnectionActivity;
				if (!SessionDatabase_Server.GetConnectionActivity(InActivityId, ConnectionActivity))
				{
					AddError(FString::Printf(TEXT("Failed to get connection activity '%s' from server database: %s"), *LexToString(InActivityId), *SessionDatabase_Server.GetLastError()));
					return false;
				}
				if (!SessionDatabase_Client.SetConnectionActivity(ConnectionActivity))
				{
					AddError(FString::Printf(TEXT("Failed to set connection activity '%s' on client database: %s"), *LexToString(InActivityId), *SessionDatabase_Client.GetLastError()));
					return false;
				}
			}
			break;

			case EConcertSyncActivityEventType::Lock:
			{
				FConcertSyncLockActivity LockActivity;
				if (!SessionDatabase_Server.GetLockActivity(InActivityId, LockActivity))
				{
					AddError(FString::Printf(TEXT("Failed to get lock activity '%s' from server database: %s"), *LexToString(InActivityId), *SessionDatabase_Server.GetLastError()));
					return false;
				}
				if (!SessionDatabase_Client.SetLockActivity(LockActivity))
				{
					AddError(FString::Printf(TEXT("Failed to set lock activity '%s' on client database: %s"), *LexToString(InActivityId), *SessionDatabase_Client.GetLastError()));
					return false;
				}
			}
			break;

			case EConcertSyncActivityEventType::Transaction:
			{
				FConcertSyncTransactionActivity TransactionActivity;
				if (!SessionDatabase_Server.GetTransactionActivity(InActivityId, TransactionActivity))
				{
					AddError(FString::Printf(TEXT("Failed to get transaction activity '%s' from server database: %s"), *LexToString(InActivityId), *SessionDatabase_Server.GetLastError()));
					return false;
				}
				if (!SessionDatabase_Client.SetTransactionActivity(TransactionActivity))
				{
					AddError(FString::Printf(TEXT("Failed to set transaction activity '%s' on client database: %s"), *LexToString(InActivityId), *SessionDatabase_Client.GetLastError()));
					return false;
				}
			}
			break;

			case EConcertSyncActivityEventType::Package:
			{
				bool bSetPackageActivitySucceeded = true;
				bool bGetPackageActivitySucceeded = SessionDatabase_Server.GetPackageActivity(InActivityId, [&](FConcertSyncActivity&& ActivityBasePart, FConcertSyncPackageEventData& ActivityEventPart)
				{
					bSetPackageActivitySucceeded = SessionDatabase_Client.SetPackageActivity(ActivityBasePart, ActivityEventPart);
				});

				if (!bGetPackageActivitySucceeded)
				{
					AddError(FString::Printf(TEXT("Failed to get package activity '%s' from server database: %s"), *LexToString(InActivityId), *SessionDatabase_Server.GetLastError()));
					return false;
				}
				if (!bSetPackageActivitySucceeded)
				{
					AddError(FString::Printf(TEXT("Failed to set package activity '%s' on client database: %s"), *LexToString(InActivityId), *SessionDatabase_Client.GetLastError()));
					return false;
				}
			}
			break;

			case EConcertSyncActivityEventType::Replication:
			{
				FConcertSyncReplicationActivity ReplicationActivity;
				if (!SessionDatabase_Server.GetReplicationActivity(InActivityId, ReplicationActivity))
				{
					AddError(FString::Printf(TEXT("Failed to get replication activity '%s' from server database: %s"), *LexToString(InActivityId), *SessionDatabase_Server.GetLastError()));
					return false;
				}
				if (!SessionDatabase_Client.SetReplicationActivity(ReplicationActivity))
				{
					AddError(FString::Printf(TEXT("Failed to set replication activity '%s' on client database: %s"), *LexToString(InActivityId), *SessionDatabase_Client.GetLastError()));
					return false;
				}
			}
			break;

			default:
				AddError(TEXT("Unhandled EConcertSyncActivityEventType when populating client database"));
				break;
			}

			return true;
		});
	}

	// Verify the two databases have the same content
	if (SessionDatabase_Server.IsValid() && SessionDatabase_Client.IsValid())
	{
		SessionDatabase_Server.EnumerateEndpoints([this, &SessionDatabase_Client](FConcertSyncEndpointIdAndData&& InEndpointData)
		{
			FConcertSyncEndpointData ClientEndpointData;
			if (!SessionDatabase_Client.GetEndpoint(InEndpointData.EndpointId, ClientEndpointData))
			{
				AddError(FString::Printf(TEXT("Failed to get endpoint '%s' from client database: %s"), *InEndpointData.EndpointId.ToString(), *SessionDatabase_Client.GetLastError()));
				return false;
			}
			return true;
		});

		SessionDatabase_Server.EnumerateActivityIdsAndEventTypes([this, &SessionDatabase_Client](const int64 InActivityId, const EConcertSyncActivityEventType InEventType)
		{
			FConcertSyncActivity ClientActivity;
			if (!SessionDatabase_Client.GetActivity(InActivityId, ClientActivity))
			{
				AddError(FString::Printf(TEXT("Failed to get activity '%s' from client database: %s"), *LexToString(InActivityId), *SessionDatabase_Client.GetLastError()));
				return false;
			}
			if (InEventType != ClientActivity.EventType)
			{
				AddError(FString::Printf(TEXT("Activity '%s' from client database was the incorrect type"), *LexToString(InActivityId)));
				return false;
			}
			return true;
		});
	}

	if (SessionDatabase_Server.IsValid() && !SessionDatabase_Server.Close())
	{
		AddError(TEXT("Failed to close server database"));
	}

	if (SessionDatabase_Client.IsValid() && !SessionDatabase_Client.Close())
	{
		AddError(TEXT("Failed to close client database"));
	}

	IFileManager::Get().DeleteDirectory(*TestSessionPath_Server, false, true);
	IFileManager::Get().DeleteDirectory(*TestSessionPath_Client, false, true);

	return true;
}
