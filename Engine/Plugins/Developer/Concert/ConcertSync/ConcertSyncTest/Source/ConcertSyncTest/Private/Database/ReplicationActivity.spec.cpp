// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncSessionDatabase.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Messages/ReplicationActivity.h"

#include "Components/StaticMeshComponent.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "Templates/UniquePtr.h"

namespace UE::ConcertSyncTests
{
	BEGIN_DEFINE_SPEC(FConcertDatabaseTest_ReplicationActivity, "VirtualProduction.Concert.Database.ReplicationActivity", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FConcertSyncSessionDatabase> Database;
	
		static FString GetDatabasePath() { return FPaths::ProjectIntermediateDir() / TEXT("ConcertDatabaseTest_Server"); }
		static FConcertSyncReplicationPayload_LeaveReplication MakeLeaveReplicationActivity(const FSoftObjectPath& ObjectPath, const FGuid& StreamId)
		{
			FConcertReplicationStream Stream;
			Stream.BaseDescription.Identifier = StreamId;
			FConcertReplicatedObjectInfo& ObjectInfo = Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(ObjectPath);
			ObjectInfo.ClassPath = UStaticMeshComponent::StaticClass();
			const TOptional<FConcertPropertyChain> Property_RelativeX = FConcertPropertyChain::CreateFromPath(*UStaticMeshComponent::StaticClass(), { TEXT("RelativeLocation"), TEXT("X") });
			if (ensure(Property_RelativeX))
			{
				ObjectInfo.PropertySelection.ReplicatedProperties.Add(*Property_RelativeX);
			}
			
			FConcertSyncReplicationPayload_LeaveReplication Payload;
			Payload.Streams.Add(Stream);
			Payload.OwnedObjects.Add(FConcertObjectInStreamID{ StreamId, ObjectPath });
				
			return Payload;
		}

		static FConcertSyncReplicationActivity MakeActivity(FConcertSyncReplicationPayload_LeaveReplication Payload, const FGuid& ClientId)
		{
			FConcertSyncReplicationActivity Activity(Payload);
			Activity.EndpointId = ClientId;
			return Activity;
		}
	END_DEFINE_SPEC(FConcertDatabaseTest_ReplicationActivity);

	/** This tests that clients respect sync control (see FConcertReplication_ChangeSyncControl). */
	void FConcertDatabaseTest_ReplicationActivity::Define()
	{
		BeforeEach([this]()
		{
			// Delete the files in case we did not clean up in a previous run (e.g. we hit a break point and then terminated the app).
			IFileManager::Get().DeleteDirectory(*GetDatabasePath(), false, true);
			
			Database = MakeUnique<FConcertSyncSessionDatabase>();
			Database->Open(GetDatabasePath());
		});

		AfterEach([this]()
		{
			Database->Close();
			IFileManager::Get().DeleteDirectory(*GetDatabasePath(), false, true);
			Database.Reset();
		});

		It("Add and retrieve FConcertSyncReplicationPayload_LeaveReplication", [this]
		{
			const FGuid StreamId = FGuid::NewGuid();
			const FSoftObjectPath StaticMeshComponent(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0"));
			
			const FConcertSyncReplicationPayload_LeaveReplication OriginalPayload = MakeLeaveReplicationActivity(StaticMeshComponent, StreamId);
			const FConcertSyncReplicationActivity OriginalActivity(OriginalPayload);
			int64 ActivityId, ReplicationEventId;
			const bool bAddedActivity = Database->AddReplicationActivity(OriginalActivity, ActivityId, ReplicationEventId);
			if (!bAddedActivity)
			{
				AddError(TEXT("Failed to store activity in database"));
				return;
			}

			FConcertSyncReplicationActivity RetrievedActivity;
			const bool bRetrievedActivity = Database->GetReplicationActivity(ActivityId, RetrievedActivity);
			if (!bRetrievedActivity)
			{
				AddError(TEXT("Failed to retrieve activity from database"));
				return;
			}

			FConcertSyncReplicationPayload_LeaveReplication RetriedPayload;
			const bool bReadStreamChange = RetrievedActivity.EventData.GetPayload(RetriedPayload);
			if (!bReadStreamChange)
			{
				AddError(TEXT("Failed to read payload as FConcertSyncReplicationPayload_StreamChange"));
				return;
			}

			TestEqual(TEXT("Stream changes equal"), OriginalPayload, RetriedPayload);
		});

		It("EnumerateReplicationActivities", [this]
		{
			const FGuid StreamId = FGuid::NewGuid();
			TArray Activities_OriginalOrder =
			{
				FConcertSyncReplicationActivity(MakeLeaveReplicationActivity(FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")), StreamId)),
				FConcertSyncReplicationActivity(MakeLeaveReplicationActivity(FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")), StreamId)),
				FConcertSyncReplicationActivity(MakeLeaveReplicationActivity(FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent2")), StreamId))
			};
			for (FConcertSyncReplicationActivity& Activity : Activities_OriginalOrder)
			{
				Database->AddReplicationActivity(Activity, Activity.ActivityId, Activity.EventId);
			}
			
			TArray<FConcertSyncReplicationActivity> Activities_EnumerationOrder;
			Database->EnumerateReplicationActivities([&Activities_EnumerationOrder](FConcertSyncReplicationActivity&& Activity)
			{
				Activities_EnumerationOrder.Emplace(MoveTemp(Activity));
				return true;
			});

			if (Activities_EnumerationOrder.Num() != Activities_OriginalOrder.Num())
			{
				AddError(TEXT("Counts do not match"));
				return;
			}
			for (int32 i = 0; i < Activities_EnumerationOrder.Num(); ++i)
			{
				TestEqual(TEXT("ActivityId"), Activities_EnumerationOrder[i].ActivityId, Activities_OriginalOrder[i].ActivityId);
				TestEqual(TEXT("EventId"), Activities_EnumerationOrder[i].EventId, Activities_OriginalOrder[i].EventId);
				TestEqual(TEXT("EventData"), Activities_EnumerationOrder[i].EventData, Activities_OriginalOrder[i].EventData);
			}
		});
		
		It("GetReplicationMaxEventIdByClientAndType", [this]
		{
			const FGuid StreamId = FGuid::NewGuid();
			const FGuid ClientOne = FGuid::NewGuid();
			const FGuid ClientTwo = FGuid::NewGuid();
			TArray Activities_OriginalOrder =
			{
				MakeActivity(MakeLeaveReplicationActivity(FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")), StreamId), ClientOne),
				MakeActivity(MakeLeaveReplicationActivity(FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")), StreamId), ClientOne),
				MakeActivity(MakeLeaveReplicationActivity(FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent2")), StreamId), ClientTwo)
			};
			for (FConcertSyncReplicationActivity& Activity : Activities_OriginalOrder)
			{
				Database->AddReplicationActivity(Activity, Activity.ActivityId, Activity.EventId);
			}
			
			int64 EventId_ClientOne = INDEX_NONE;
			int64 EventId_ClientTwo = INDEX_NONE;
			int64 EventId_Invalid = INDEX_NONE;
			const bool bSuccess_Query1 = Database->GetReplicationMaxEventIdByClientAndType(ClientOne, EConcertSyncReplicationActivityType::LeaveReplication, EventId_ClientOne);
			const bool bSuccess_Query2 = Database->GetReplicationMaxEventIdByClientAndType(ClientTwo, EConcertSyncReplicationActivityType::LeaveReplication, EventId_ClientTwo);
			const bool bSuccess_InvalidQuery = Database->GetReplicationMaxEventIdByClientAndType(FGuid::NewGuid(), EConcertSyncReplicationActivityType::LeaveReplication, EventId_Invalid);
			
			TestTrue(TEXT("Query 1"), bSuccess_Query1);
			TestEqual(TEXT("Client 1 Max EventId"), EventId_ClientOne, Activities_OriginalOrder[1].EventId);
			TestTrue(TEXT("Query 2"), bSuccess_Query2);
			TestEqual(TEXT("Client 2 Max EventId"), EventId_ClientTwo, Activities_OriginalOrder[2].EventId);
			TestTrue(TEXT("Invalid query"), bSuccess_InvalidQuery);
			// Sql query returns NULL here (since nothing is found).
			// Under the hood, sqlite3_value_int64 is used to get the value of the returned column... NULL is converted to 0 by sqlite3_value_int64.
			TestEqual(TEXT("Unfound returns NULL"), EventId_Invalid, static_cast<int64>(0));
		});
	}
}
