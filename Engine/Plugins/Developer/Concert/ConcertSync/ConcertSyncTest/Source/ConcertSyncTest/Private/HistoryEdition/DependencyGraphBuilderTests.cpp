// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogGlobal.h"

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "HistoryEdition/DebugDependencyGraph.h"
#include "HistoryEdition/ActivityNode.h"
#include "HistoryEdition/DependencyGraphBuilder.h"
#include "HistoryTestUtil.h"
#include "RenameEditAndDeleteMapsFlow.h"
#include "Util/ActivityBuilder.h"
#include "Util/ScopedSessionDatabaseWithEndpoint.h"

namespace UE::ConcertSyncTests::RenameEditAndDeleteMapsFlowTest
{
	/** Validates that the graph reflects the expected dependencies. */
	bool ValidateExpectedDependencies(FAutomationTestBase& Test, const TTestActivityArray<int64>& ActivityMappings, const ConcertSyncCore::FActivityDependencyGraph& Graph);
	/** Validates that each activity has a node in the dependency graph. */
	TTestActivityArray<ConcertSyncCore::FActivityNodeID> ValidateEachActivityHasNode(FAutomationTestBase& Test, const TTestActivityArray<int64>& ActivityMappings, const ConcertSyncCore::FActivityDependencyGraph& Graph);
	
	/**
	 * Builds the dependency graph from a typical sequence of events.
	 *
	 * Sequence of user actions:
	 *	1 Create map Foo
	 *	2 Add actor A
	 *	3 Rename actor A
	 *	4 Edit actor A
	 *	5 Rename map to Bar
	 *	6 Edit actor A
	 *	7 Delete map Bar
	 *	8 Create map Bar
	 *
	 *	The dependency graph should look like this:
	 *	2 -> 1 (PackageCreation)
	 *	3 -> 2 (EditPossiblyDependsOnPackage)
	 *	4 -> 3 (EditPossiblyDependsOnPackage)
	 *	5 -> 1 (PackageCreation)
	 *	6 -> 5 (PackageRename)
	 *	7 -> 5 (PackageRename)
	 *	8 -> 5 (PackageRemoval)
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRenameEditAndDeleteMapsFlowTest, "VirtualProduction.Concert.History.BuildGraph.RenameEditAndDeleteMapsFlow", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FRenameEditAndDeleteMapsFlowTest::RunTest(const FString& Parameters)
	{
		FScopedSessionDatabaseWithEndpoint SessionDatabase(*this);
		const TTestActivityArray<int64> Activities = CreateActivityHistory(SessionDatabase, SessionDatabase.GetEndpoint());
		
		const ConcertSyncCore::FActivityDependencyGraph DependencyGraph = ConcertSyncCore::BuildDependencyGraphFrom(SessionDatabase);
		UE_LOGF(LogConcert, Log, "%ls tested graph in Graphviz format:\n\n%ls", *GetTestFullName(), *ConcertSyncCore::Graphviz::ExportToGraphviz(DependencyGraph, SessionDatabase));
		return ValidateExpectedDependencies(*this, Activities, DependencyGraph);
	}
	
	bool ValidateExpectedDependencies(FAutomationTestBase& Test, const TTestActivityArray<int64>& ActivityMappings, const ConcertSyncCore::FActivityDependencyGraph& Graph)
	{
		using namespace ConcertSyncCore;
		
		const TTestActivityArray<FActivityNodeID> Nodes
			= ValidateEachActivityHasNode(Test, ActivityMappings, Graph);

		// TODO: Update this to respect PackageEdited
		
		// 1 Create map Foo
		{
			Test.TestFalse(TEXT("_1_NewPackageFoo has no dependencies"), Graph.GetNodeById(Nodes[_1_NewPackageFoo]).HasAnyDependency());
			Test.TestTrue(TEXT("_1_NewPackageFoo has correct node flags"), Graph.GetNodeById(Nodes[_1_NewPackageFoo]).GetNodeFlags() == EActivityNodeFlags::None);
			
			Test.TestTrue(TEXT("_2_SavePackageFoo depends on _1_NewPackageFoo"), Graph.GetNodeById(Nodes[_2_SavePackageFoo]).DependsOnActivity(ActivityMappings[_1_NewPackageFoo], Graph, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_2_SavePackageFoo has correct node flags"), Graph.GetNodeById(Nodes[_2_SavePackageFoo]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestEqual(TEXT("_2_SavePackageFoo only depends on _1_NewPackageFoo"), Graph.GetNodeById(Nodes[_2_SavePackageFoo]).GetDependencies().Num(), 1);
		}

		// 2 Add actor A
		{
			Test.TestTrue(TEXT("_3_AddActor depends on _1_NewPackageFoo"), Graph.GetNodeById(Nodes[_3_AddActor]).DependsOnActivity(ActivityMappings[_1_NewPackageFoo], Graph, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_3_AddActor has correct node flags"), Graph.GetNodeById(Nodes[_3_AddActor]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestEqual(TEXT("_3_AddActor only depends on _1_NewPackageFoo"), Graph.GetNodeById(Nodes[_3_AddActor]).GetDependencies().Num(), 1);
		}

		// 3 Rename actor A
		{
			// It must be a EDependencyStrength::HardDependency because you cannot edit the actor without having created it 
			Test.TestTrue(TEXT("_4_RenameActor depends on _3_AddActor"), Graph.GetNodeById(Nodes[_4_RenameActor]).DependsOnActivity(ActivityMappings[_3_AddActor], Graph, EActivityDependencyReason::SubobjectCreation, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_4_RenameActor has correct node flags"), Graph.GetNodeById(Nodes[_4_RenameActor]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestEqual(TEXT("_4_RenameActor only depends on _3_AddActor"), Graph.GetNodeById(Nodes[_4_RenameActor]).GetDependencies().Num(), 1);
		}

		// 4 Edit actor A
		{
			// This activity must have a EDependencyStrength::HardDependency to _2_AddActor because the edit cannot happen without having created the actor
			Test.TestTrue(TEXT("_5_EditActor depends on _3_AddActor"), Graph.GetNodeById(Nodes[_5_EditActor]).DependsOnActivity(ActivityMappings[_3_AddActor], Graph, EActivityDependencyReason::SubobjectCreation, EDependencyStrength::HardDependency));
			// The previous edit might have affected us (e.g. this activity may have executed the construction script)
			// Note: This should not have a hard dependency on having renamed the actor because a rename is just a property change of ActorLabel.
			Test.TestTrue(TEXT("_5_EditActor depends on _4_RenameActor"), Graph.GetNodeById(Nodes[_5_EditActor]).DependsOnActivity(ActivityMappings[_4_RenameActor], Graph, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::PossibleDependency));
			Test.TestTrue(TEXT("_5_EditActor has correct node flags"), Graph.GetNodeById(Nodes[_5_EditActor]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestEqual(TEXT("_5_EditActor only depends on _3_AddActor and _4_RenameActor"), Graph.GetNodeById(Nodes[_5_EditActor]).GetDependencies().Num(), 2);
		}

		// 5 Rename map to Bar
		{
			Test.TestTrue(TEXT("_6_SavePackageBar has correct node flags"), Graph.GetNodeById(Nodes[_6_SavePackageBar]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestTrue(TEXT("_6_SavePackageBar depends on _3_AddActor"), Graph.GetNodeById(Nodes[_6_SavePackageBar]).DependsOnNode(Nodes[_3_AddActor], EActivityDependencyReason::PackageEdited, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_6_SavePackageBar depends on _4_RenameActor"), Graph.GetNodeById(Nodes[_6_SavePackageBar]).DependsOnNode(Nodes[_4_RenameActor], EActivityDependencyReason::PackageEdited, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_6_SavePackageBar depends on _5_EditActor"), Graph.GetNodeById(Nodes[_6_SavePackageBar]).DependsOnNode(Nodes[_5_EditActor], EActivityDependencyReason::PackageEdited, EDependencyStrength::HardDependency));
			Test.TestEqual(TEXT("_6_SavePackageBar depends on _3_AddActor, _4_RenameActor, and _5_EditActor"), Graph.GetNodeById(Nodes[_6_SavePackageBar]).GetDependencies().Num(), 3);
			
			Test.TestTrue(TEXT("_7_RenameFooToBar depends on _1_NewPackageFoo"), Graph.GetNodeById(Nodes[_7_RenameFooToBar]).DependsOnActivity(ActivityMappings[_1_NewPackageFoo], Graph, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_7_RenameFooToBar depends on _6_SavePackageBar"), Graph.GetNodeById(Nodes[_7_RenameFooToBar]).DependsOnActivity(ActivityMappings[_6_SavePackageBar], Graph, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_7_RenameFooToBar has correct node flags"), Graph.GetNodeById(Nodes[_7_RenameFooToBar]).GetNodeFlags() == EActivityNodeFlags::RenameActivity);
			Test.TestEqual(TEXT("_7_RenameFooToBar depends only on _1_NewPackageFoo and _6_SavePackageBar"), Graph.GetNodeById(Nodes[_7_RenameFooToBar]).GetDependencies().Num(), 2);
		}

		// 6 Edit actor A
		{
			Test.TestTrue(TEXT("_8_EditActor depends on _7_RenameFooToBar"), Graph.GetNodeById(Nodes[_8_EditActor]).DependsOnActivity(ActivityMappings[_7_RenameFooToBar], Graph, EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_8_EditActor has correct node flags"), Graph.GetNodeById(Nodes[_8_EditActor]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestEqual(TEXT("_8_EditActor only depends on _7_RenameFooToBar"), Graph.GetNodeById(Nodes[_8_EditActor]).GetDependencies().Num(), 1);
		}

		// 7 Delete map Bar
		{
			Test.TestTrue(TEXT("_9_DeleteBar depends on _7_RenameFooToBar"), Graph.GetNodeById(Nodes[_9_DeleteBar]).DependsOnActivity(ActivityMappings[_7_RenameFooToBar], Graph, EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_9_DeleteBar has correct node flags"), Graph.GetNodeById(Nodes[_9_DeleteBar]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestEqual(TEXT("_9_DeleteBar depends only on _7_RenameFooToBar"), Graph.GetNodeById(Nodes[_9_DeleteBar]).GetDependencies().Num(), 1);
		}

		// 8 Create map Bar
		{
			Test.TestTrue(TEXT("_10_NewPackageFoo depends on _9_DeleteBar"), Graph.GetNodeById(Nodes[_10_NewPackageFoo]).DependsOnActivity(ActivityMappings[_9_DeleteBar], Graph, EActivityDependencyReason::PackageRemoval, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_10_NewPackageFoo has correct node flags"), Graph.GetNodeById(Nodes[_10_NewPackageFoo]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestEqual(TEXT("_10_NewPackageFoo depends only on _9_DeleteBar"), Graph.GetNodeById(Nodes[_10_NewPackageFoo]).GetDependencies().Num(), 1);
			
			Test.TestTrue(TEXT("_11_SavePackageFoo depends on _10_NewPackageFoo"), Graph.GetNodeById(Nodes[_11_SavePackageFoo]).DependsOnActivity(ActivityMappings[_10_NewPackageFoo], Graph, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			Test.TestTrue(TEXT("_11_SavePackageFoo has correct node flags"), Graph.GetNodeById(Nodes[_11_SavePackageFoo]).GetNodeFlags() == EActivityNodeFlags::None);
			Test.TestEqual(TEXT("_11_SavePackageFoo depends only on _10_NewPackageFoo"), Graph.GetNodeById(Nodes[_11_SavePackageFoo]).GetDependencies().Num(), 1);
		}
		
		return true;
	}

	TTestActivityArray<ConcertSyncCore::FActivityNodeID> ValidateEachActivityHasNode(FAutomationTestBase& Test, const TTestActivityArray<int64>& ActivityMappings, const ConcertSyncCore::FActivityDependencyGraph& Graph)
	{
		using namespace ConcertSyncCore;
		
		TTestActivityArray<FActivityNodeID> ActivityNodes;
		ActivityNodes.SetNumZeroed(ActivityCount);
		for (int32 ActivityType = 0; ActivityType < ActivityCount; ++ActivityType)
		{
			const int64 ActivityId = ActivityMappings[ActivityType];
			const TOptional<FActivityNodeID> NodeID = Graph.FindNodeByActivity(ActivityId);
			if (!NodeID.IsSet())
			{
				Test.AddError(FString::Printf(TEXT("No node generated for activity %s"), *LexToString(static_cast<ETestActivity>(ActivityType))));
				continue;
			}
			
			const FActivityNode& Node = Graph.GetNodeById(*NodeID);
			if (!NodeID.IsSet())
			{
				Test.AddError(FString::Printf(TEXT("Graph has invalid state. Node ID %lld is invalid for activity %s"), NodeID->ID, *LexToString(static_cast<ETestActivity>(ActivityType))));
				continue;
			}
			
			ActivityNodes[ActivityType] = *NodeID;
		}

		return ActivityNodes;
	}
}

namespace UE::ConcertSyncTests::DeletingAndRecreatingActorIsHardDependency
{
	enum ETestActivity
	{
		CreateActor,
		DeleteActor,
		RecreateActor,

		ActivityCount
	};
	
	TArray<FActivityID> FillDatabase(FScopedSessionDatabaseWithEndpoint& Database);
	FConcertExportedObject CreateActorMetaData(FName OuterLevelPath);
	
	/**
	 * 1. Create actor A
	 * 2. Delete Actor A
	 * 3. Re-create actor A.
	 *
	 * 3 > 2 is a hard dependency (removing 2 would result in attempting to create the actor twice, which is invalid).
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDeletingAndRecreatingActorIsHardDependency, "VirtualProduction.Concert.History.BuildGraph.DeletingAndRecreatingActorIsHardDependency", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FDeletingAndRecreatingActorIsHardDependency::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncCore;
		
		FScopedSessionDatabaseWithEndpoint SessionDatabase(*this);
		const TArray<FActivityID> TestActivities = FillDatabase(SessionDatabase);

		const FActivityDependencyGraph DependencyGraph = BuildDependencyGraphFrom(SessionDatabase);
		UE_LOGF(LogConcert, Log, "%ls tested graph in Graphviz format:\n\n%ls", *GetTestFullName(), *ConcertSyncCore::Graphviz::ExportToGraphviz(DependencyGraph, SessionDatabase));

		const TOptional<FActivityNodeID> CreateActorNodeID = DependencyGraph.FindNodeByActivity(TestActivities[CreateActor]);
		const TOptional<FActivityNodeID> DeleteActorNodeID = DependencyGraph.FindNodeByActivity(TestActivities[DeleteActor]);
		const TOptional<FActivityNodeID> RecreateActorNodeID = DependencyGraph.FindNodeByActivity(TestActivities[RecreateActor]);
		if (!CreateActorNodeID || !DeleteActorNodeID || !RecreateActorNodeID)
		{
			AddError(TEXT("Activities not registered"));
			return false;
		}

		const FActivityNode& CreatedActorNode = DependencyGraph.GetNodeById(*CreateActorNodeID);
		const FActivityNode& DeleteActorNode = DependencyGraph.GetNodeById(*DeleteActorNodeID);
		const FActivityNode& RecreateActorNode = DependencyGraph.GetNodeById(*RecreateActorNodeID);

		TestEqual(TEXT("CreatedActorNode->GetDependencies().Num() == 0"), CreatedActorNode.GetDependencies().Num(), 0);
		TestEqual(TEXT("DeleteActorNode->GetDependencies().Num() == 1"), DeleteActorNode.GetDependencies().Num(), 1);
		TestEqual(TEXT("RecreateActorNode->GetDependencies().Num() == 1"), RecreateActorNode.GetDependencies().Num(), 1);

		TestTrue(TEXT("DeleteActorNode depends on CreatedActorNode"), DeleteActorNode.DependsOnActivity(TestActivities[CreateActor], DependencyGraph, EActivityDependencyReason::SubobjectCreation, EDependencyStrength::HardDependency));
		TestTrue(TEXT("RecreateActorNode depends on DeleteActorNode"), RecreateActorNode.DependsOnActivity(TestActivities[DeleteActor], DependencyGraph, EActivityDependencyReason::SubobjectRemoval, EDependencyStrength::HardDependency));

		return true;
	}
	
	TArray<FActivityID> FillDatabase(FScopedSessionDatabaseWithEndpoint& SessionDatabase)
	{
		TArray<FActivityID> ActivityIDs;
		ActivityIDs.SetNumUninitialized(ActivityCount);

		const FName FooLevel = TEXT("/Game/Foo");
		int64 Dummy;
		
		// 1 Create actor
		{
			FConcertSyncTransactionActivity CreateActorActivity;
			CreateActorActivity.EndpointId = SessionDatabase.GetEndpoint();
			CreateActorActivity.EventData.Transaction.TransactionId = FGuid::NewGuid();
			CreateActorActivity.EventData.Transaction.OperationId = FGuid::NewGuid();
			FConcertExportedObject Actor = CreateActorMetaData(FooLevel);
			Actor.ObjectData.bAllowCreate = true;
			CreateActorActivity.EventData.Transaction.ExportedObjects = { Actor };
			CreateActorActivity.EventData.Transaction.ModifiedPackages = { FooLevel };
			SessionDatabase.GetTransactionMaxEventId(CreateActorActivity.EventId);
			SessionDatabase.AddTransactionActivity(CreateActorActivity, ActivityIDs[CreateActor], Dummy);
		}
		
		// 2 Delete actor
		{
			FConcertSyncTransactionActivity RemoveActorActivity;
			RemoveActorActivity.EndpointId = SessionDatabase.GetEndpoint();
			RemoveActorActivity.EventData.Transaction.TransactionId = FGuid::NewGuid();
			RemoveActorActivity.EventData.Transaction.OperationId = FGuid::NewGuid();
			FConcertExportedObject Actor = CreateActorMetaData(FooLevel);
			Actor.ObjectData.bIsPendingKill = true;
			RemoveActorActivity.EventData.Transaction.ExportedObjects = { Actor };
			RemoveActorActivity.EventData.Transaction.ModifiedPackages = { FooLevel };
			SessionDatabase.GetTransactionMaxEventId(RemoveActorActivity.EventId);
			SessionDatabase.AddTransactionActivity(RemoveActorActivity, ActivityIDs[DeleteActor], Dummy);
		}
		
		// 3 Re-create actor
		{
			FConcertSyncTransactionActivity CreateActorActivity;
			CreateActorActivity.EndpointId = SessionDatabase.GetEndpoint();
			CreateActorActivity.EventData.Transaction.TransactionId = FGuid::NewGuid();
			CreateActorActivity.EventData.Transaction.OperationId = FGuid::NewGuid();
			FConcertExportedObject Actor = CreateActorMetaData(FooLevel);
			Actor.ObjectData.bAllowCreate = true;
			CreateActorActivity.EventData.Transaction.ExportedObjects = { Actor };
			CreateActorActivity.EventData.Transaction.ModifiedPackages = { FooLevel };
			SessionDatabase.GetTransactionMaxEventId(CreateActorActivity.EventId);
			SessionDatabase.AddTransactionActivity(CreateActorActivity, ActivityIDs[RecreateActor], Dummy);
		}
		
		return ActivityIDs;
	}
	
	FConcertExportedObject CreateActorMetaData(FName OuterLevelPath)
	{
		FConcertExportedObject Result;
		Result.ObjectId.ObjectName = TEXT("SomeTestActorName42");
		Result.ObjectId.ObjectPackageName = OuterLevelPath;
		Result.ObjectId.ObjectOuterPathName = *FString::Printf(TEXT("%s:PersistentLevel"), *OuterLevelPath.ToString());
		Result.ObjectId.ObjectClassPathName = TEXT("/Script/Engine.StaticMeshActor");
		return Result;
	}
}

namespace UE::ConcertSyncTests::IndirectPackageDependencyTest
{
	/**
	 * This tests that potential indirect dependencies are handled.
	 *
	 * Sequence of user actions:
	 *  1 Create data asset A
	 *  2 Make actor reference A
	 *  3 Edit data asset
	 *  4 Edit actor
	 *
	 * The dependency graph should look like this:
	 *  2 -> 1 (PackageCreation)
	 *  3 -> 1 (PackageCreation)
	 *  4 -> 1 (EditPossiblyDependsOnPackage)
	 *  4 -> 2 (EditPossiblyDependsOnPackage)
	 *
	 * This is relevant because the actor's construction script may depend query data from the data asset.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIndirectPackageDependencyTest, "VirtualProduction.Concert.History.BuildGraph.IndirectPackageDependencyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FIndirectPackageDependencyTest::RunTest(const FString& Parameters)
	{
		// TODO:
		return true;
	}
}

namespace UE::ConcertSyncTests::PackageEditedDependencyTest
{
	enum ETestActivity
	{
		// Normal flow: saving depends on transaction
		_1_CreateActor,
		_2_SaveFoo,
		// Saving after without transactions since last save has no dependencies
		_3_SaveFoo,
		_4_CreateActor,

		// Renaming a package inherits the dependencies to the transactions that occured before the rename
		_5_SaveBar,
		_6_RenameFooToBar,

		// Creating an equally named package after renaming it causes no dependencies
		_7_NewMapFoo,
		_8_SaveMapFoo,
		_9_CreateActor,
		_10_SaveFoo,
		_11_DeleteFoo,

		// Creating an equally named package after deleting it causes no dependencies
		_12_NewMapFoo,
		_13_SaveMapFoo,
		_14_CreateActor,
		_15_SaveFoo,

		ActivityCount
	};
	
	TArray<FActivityID> FillDatabase(FScopedSessionDatabaseWithEndpoint& SessionDatabase);
	
	/**
	 * This tests that EActivityDependencyReason::PackageEdited dependencies are discovered correctly.
	 *
	 * Sequence of user actions:
	 *  1 Create actor
	 *  2 Save
	 *  3 Save
	 *  4 Create actor
	 *  5 Rename to Bar
	 *  6 Create new map Foo
	 *  7 Create actor
	 *  8 Save Foo
	 *  9 Delete Foo
	 *  10 New map Foo
	 *  11 Save Foo
	 *  12 Create actor
	 *  13 Save Foo
	 *
	 *  Only these dependencies should have PackageEdited as reason:
	 *  2 -> 1
	 *  5 -> 4 (the save before the rename has the dependency)
	 *  8 -> 7
	 *  13 -> 12
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPackageEditedDependencyTest, "VirtualProduction.Concert.History.BuildGraph.PackageEditedDependencyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FPackageEditedDependencyTest::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncCore;
		
		FScopedSessionDatabaseWithEndpoint SessionDatabase(*this);
		TArray<FActivityID> Activities = FillDatabase(SessionDatabase);
		
		const FActivityDependencyGraph DependencyGraph = BuildDependencyGraphFrom(SessionDatabase);
		UE_LOGF(LogConcert, Log, "%ls tested graph in Graphviz format:\n\n%ls", *GetTestFullName(), *ConcertSyncCore::Graphviz::ExportToGraphviz(DependencyGraph, SessionDatabase));
		TArray<FActivityNodeID> Nodes = ConcertSyncTests::ValidateEachActivityHasNode(*this, Activities, DependencyGraph, ActivityCount, [](const uint32 ActivityType){ return FString::FromInt(ActivityType); });

		// _1_CreateActor
		{
			TestTrue(TEXT("_1_CreateActor: Affects first _2_SaveFoo"), DependencyGraph.GetNodeById(Nodes[_1_CreateActor]).AffectsActivity(Activities[_2_SaveFoo], DependencyGraph));
			TestTrue(TEXT("_1_CreateActor: Affects first _2_SaveFoo"), DependencyGraph.GetNodeById(Nodes[_1_CreateActor]).AffectsActivity(Activities[_4_CreateActor], DependencyGraph));
			TestEqual(TEXT("_1_CreateActor: Affects only _2_SaveFoo"), DependencyGraph.GetNodeById(Nodes[_1_CreateActor]).GetAffectedChildren().Num(), 2);
		}
		// _2_SaveFoo
		{
			TestTrue(TEXT("_2_SaveFoo: Depends on _1_CreateActor"), DependencyGraph.GetNodeById(Nodes[_2_SaveFoo]).DependsOnNode(Nodes[_1_CreateActor], EActivityDependencyReason::PackageEdited, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_2_SaveFoo: Depends only on _1_CreateActor"), DependencyGraph.GetNodeById(Nodes[_2_SaveFoo]).GetDependencies().Num(), 1);
		}
		// _3_SaveFoo
		{
			TestEqual(TEXT("_3_SaveFoo: Depends on nothing"), DependencyGraph.GetNodeById(Nodes[_3_SaveFoo]).GetDependencies().Num(), 0);
		}
		// _4_CreateActor
		{
			TestTrue(TEXT("_4_CreateActor: Depends on _1_CreateActor"), DependencyGraph.GetNodeById(Nodes[_4_CreateActor]).DependsOnNode(Nodes[_1_CreateActor], EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::PossibleDependency));
			TestEqual(TEXT("_4_CreateActor: Depends only on _1_CreateActor"), DependencyGraph.GetNodeById(Nodes[_4_CreateActor]).GetDependencies().Num(), 1);
		}
		// _5_SaveFoo2
		{
			TestTrue(TEXT("_5_SaveFoo2: Depends on _4_CreateActor"), DependencyGraph.GetNodeById(Nodes[_5_SaveBar]).DependsOnNode(Nodes[_4_CreateActor], EActivityDependencyReason::PackageEdited, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_5_SaveFoo2: Depends only on _4_CreateActor"), DependencyGraph.GetNodeById(Nodes[_5_SaveBar]).GetDependencies().Num(), 1);
		}
		// _6_RenameFooToBar
		{
			TestTrue(TEXT("_6_RenameFooToBar: Depends on _3_SaveFoo"), DependencyGraph.GetNodeById(Nodes[_6_RenameFooToBar]).DependsOnNode(Nodes[_3_SaveFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestTrue(TEXT("_6_RenameFooToBar: Depends on _5_SaveBar"), DependencyGraph.GetNodeById(Nodes[_6_RenameFooToBar]).DependsOnNode(Nodes[_5_SaveBar], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_6_RenameFooToBar: Depends only on _3_SaveFoo and _5_SaveBar"), DependencyGraph.GetNodeById(Nodes[_6_RenameFooToBar]).GetDependencies().Num(), 2);
		}
		// _7_NewMapFoo
		{
			TestEqual(TEXT("_7_NewMapFoo: Depends on nothing"), DependencyGraph.GetNodeById(Nodes[_7_NewMapFoo]).GetDependencies().Num(), 0);
		}
		// _8_SaveMapFoo
		{
			TestTrue(TEXT("_8_SaveMapFoo: Depends on _7_NewMapBar"), DependencyGraph.GetNodeById(Nodes[_8_SaveMapFoo]).DependsOnNode(Nodes[_7_NewMapFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_8_SaveMapFoo: Depends only on _7_NewMapBar"), DependencyGraph.GetNodeById(Nodes[_8_SaveMapFoo]).GetDependencies().Num(), 1);
		}
		// _9_CreateActor
		{
			TestTrue(TEXT("_9_CreateActor: Depends on _7_NewMapBar"), DependencyGraph.GetNodeById(Nodes[_9_CreateActor]).DependsOnNode(Nodes[_7_NewMapFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_9_CreateActor: Depends only on _7_NewMapBar"), DependencyGraph.GetNodeById(Nodes[_9_CreateActor]).GetDependencies().Num(), 1);
		}
		// _10_SaveFoo
		{
			TestTrue(TEXT("_10_SaveFoo: Depends on _7_NewMapFoo"), DependencyGraph.GetNodeById(Nodes[_10_SaveFoo]).DependsOnNode(Nodes[_7_NewMapFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestTrue(TEXT("_10_SaveFoo: Depends on _9_CreateActor"), DependencyGraph.GetNodeById(Nodes[_10_SaveFoo]).DependsOnNode(Nodes[_9_CreateActor], EActivityDependencyReason::PackageEdited, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_10_SaveFoo: Depends only on _7_NewMapFoo and _9_CreateActor"), DependencyGraph.GetNodeById(Nodes[_10_SaveFoo]).GetDependencies().Num(), 2);
		}
		// _11_DeleteFoo
		{
			TestTrue(TEXT("_11_DeleteFoo: Depends on _7_NewMapFoo"), DependencyGraph.GetNodeById(Nodes[_11_DeleteFoo]).DependsOnNode(Nodes[_7_NewMapFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_11_DeleteFoo: Depends only on _7_NewMapFoo"), DependencyGraph.GetNodeById(Nodes[_11_DeleteFoo]).GetDependencies().Num(), 1);
		}
		// _12_NewMapFoo
		{
			TestTrue(TEXT("_12_NewMapFoo: Depends on _11_DeleteFoo"), DependencyGraph.GetNodeById(Nodes[_12_NewMapFoo]).DependsOnNode(Nodes[_11_DeleteFoo], EActivityDependencyReason::PackageRemoval, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_12_NewMapFoo: Depends only on _11_DeleteFoo"), DependencyGraph.GetNodeById(Nodes[_12_NewMapFoo]).GetDependencies().Num(), 1);
		}
		// _13_SaveMapFoo
		{
			TestTrue(TEXT("_13_SaveMapFoo: Depends on _12_NewMapFoo"), DependencyGraph.GetNodeById(Nodes[_13_SaveMapFoo]).DependsOnNode(Nodes[_12_NewMapFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_13_SaveMapFoo: Depends only on _12_NewMapFoo"), DependencyGraph.GetNodeById(Nodes[_13_SaveMapFoo]).GetDependencies().Num(), 1);
		}
		// _14_CreateActor
		{
			TestTrue(TEXT("_14_CreateActor: Depends on _12_NewMapFoo"), DependencyGraph.GetNodeById(Nodes[_14_CreateActor]).DependsOnNode(Nodes[_12_NewMapFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_14_CreateActor: Depends only on _12_NewMapFoo"), DependencyGraph.GetNodeById(Nodes[_14_CreateActor]).GetDependencies().Num(), 1);
		}
		// _15_SaveFoo
		{
			TestTrue(TEXT("_15_SaveFoo: Depends on _12_NewMapFoo"), DependencyGraph.GetNodeById(Nodes[_15_SaveFoo]).DependsOnNode(Nodes[_12_NewMapFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestTrue(TEXT("_15_SaveFoo: Depends on _14_CreateActor"), DependencyGraph.GetNodeById(Nodes[_15_SaveFoo]).DependsOnNode(Nodes[_14_CreateActor], EActivityDependencyReason::PackageEdited, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_15_SaveFoo: Depends only on _12_NewMapFoo and _14_CreateActor"), DependencyGraph.GetNodeById(Nodes[_15_SaveFoo]).GetDependencies().Num(), 2);
		}
		
		return true;
	}

	TArray<FActivityID> FillDatabase(FScopedSessionDatabaseWithEndpoint& SessionDatabase)
	{
		const FName FooLevel = TEXT("/Game/Foo");
		const FName BarLevel = TEXT("/Game/Bar");
		FActivityBuilder ActivityBuilder(SessionDatabase, ActivityCount);
		
		ActivityBuilder.CreateActor(FooLevel, _1_CreateActor, TEXT("FirstActor"));
		ActivityBuilder.SaveMap(FooLevel, _2_SaveFoo);
		ActivityBuilder.SaveMap(FooLevel, _3_SaveFoo);
		ActivityBuilder.CreateActor(FooLevel, _4_CreateActor, TEXT("SecondActor"));
		ActivityBuilder.RenameMap(FooLevel, BarLevel, _5_SaveBar, _6_RenameFooToBar);
		ActivityBuilder.NewMap(FooLevel, _7_NewMapFoo);
		ActivityBuilder.SaveMap(FooLevel, _8_SaveMapFoo);
		ActivityBuilder.CreateActor(FooLevel, _9_CreateActor, TEXT("ThirdActor"));
		ActivityBuilder.SaveMap(FooLevel, _10_SaveFoo);
		ActivityBuilder.DeleteMap(FooLevel, _11_DeleteFoo);
		ActivityBuilder.NewMap(FooLevel, _12_NewMapFoo);
		ActivityBuilder.SaveMap(FooLevel, _13_SaveMapFoo);
		ActivityBuilder.CreateActor(FooLevel, _14_CreateActor);
		ActivityBuilder.SaveMap(FooLevel, _15_SaveFoo);
		
		return ActivityBuilder.GetActivities();
	}
}

namespace UE::ConcertSyncTests::RenamingDependencyEdgeCases
{
	enum ETestActivity
	{
		_1_AddFoo,
		_2_AddBar,
		_3_DeleteFoo,
		_4_DeleteBar,

		// The goal is to test that _7_RenameFooToBar depends on _5_AddFoo and _6_SaveBar (instead of _1_AddFoo nor _2_AddBar)
		_5_AddFoo,
		_6_SaveBar,
		_7_RenameFooToBar,

		// The goal is to test that _11_RenameBarToFoo depends on _9_AddBar and _10_SaveFoo (instead of _1_AddFoo, _5_AddFoo nor _2_AddBar, _7_RenameFooToBar
		_8_SaveFoo,
		_9_RenameBarToFoo,

		ActivityCount
	};
	
	TArray<FActivityID> FillDatabase(FScopedSessionDatabaseWithEndpoint& SessionDatabase);
	
	/**
	 * This tests that renaming dependencies are correctly set in some edge cases.
	 *
	 * User actions:
	 *  1 Add Foo
	 *  2 Add Bar
	 *  3 Delete Foo
	 *  4 Delete Bar
	 *  5 Add Foo
	 *  6 Rename Foo to Bar
	 *  7 Delete Bar
	 *  8 Add Bar
	 *  9 Rename Bar to Foo
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRenamingDependencyEdgeCaseTests, "VirtualProduction.Concert.History.BuildGraph.RenamingDependencyEdgeCaseTests", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FRenamingDependencyEdgeCaseTests::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncCore;
		
		FScopedSessionDatabaseWithEndpoint SessionDatabase(*this);
		TArray<FActivityID> Activities = FillDatabase(SessionDatabase);
		
		const FActivityDependencyGraph DependencyGraph = BuildDependencyGraphFrom(SessionDatabase);
		UE_LOGF(LogConcert, Log, "%ls tested graph in Graphviz format:\n\n%ls", *GetTestFullName(), *ConcertSyncCore::Graphviz::ExportToGraphviz(DependencyGraph, SessionDatabase));
		TArray<FActivityNodeID> Nodes = ConcertSyncTests::ValidateEachActivityHasNode(*this, Activities, DependencyGraph, ActivityCount, [](const uint32 ActivityType){ return FString::FromInt(ActivityType); });

		// _7_RenameFooToBar
		{
			TestTrue(TEXT("_7_RenameFooToBar > _5_AddFoo"), DependencyGraph.GetNodeById(Nodes[_7_RenameFooToBar]).DependsOnNode(Nodes[_5_AddFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestTrue(TEXT("_7_RenameFooToBar > _6_SaveBar"), DependencyGraph.GetNodeById(Nodes[_7_RenameFooToBar]).DependsOnNode(Nodes[_6_SaveBar], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_7_RenameFooToBar only depends on _5_AddFoo and _6_SaveBar"), DependencyGraph.GetNodeById(Nodes[_7_RenameFooToBar]).GetDependencies().Num(), 2);
		}

		// _9_RenameBarToFoo
		{
			TestTrue(TEXT("_9_RenameBarToFoo > _7_RenameFooToBar"), DependencyGraph.GetNodeById(Nodes[_9_RenameBarToFoo]).DependsOnNode(Nodes[_7_RenameFooToBar], EActivityDependencyReason::PackageRename, EDependencyStrength::HardDependency));
			TestTrue(TEXT("_9_RenameBarToFoo > _8_SaveFoo"), DependencyGraph.GetNodeById(Nodes[_9_RenameBarToFoo]).DependsOnNode(Nodes[_8_SaveFoo], EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
			TestEqual(TEXT("_9_RenameBarToFoo only depends on _7_RenameFooToBar and _8_SaveFoo"), DependencyGraph.GetNodeById(Nodes[_9_RenameBarToFoo]).GetDependencies().Num(), 2);
		}
		
		return true;
	}
	
	TArray<FActivityID> FillDatabase(FScopedSessionDatabaseWithEndpoint& SessionDatabase)
	{
		const FName FooLevel = TEXT("/Game/Foo");
		const FName BarLevel = TEXT("/Game/Bar");
		FActivityBuilder ActivityBuilder(SessionDatabase, ActivityCount);

		ActivityBuilder.NewMap(FooLevel, _1_AddFoo);
		ActivityBuilder.NewMap(BarLevel, _2_AddBar);
		ActivityBuilder.DeleteMap(FooLevel, _3_DeleteFoo);
		ActivityBuilder.DeleteMap(BarLevel, _4_DeleteBar);
		ActivityBuilder.NewMap(FooLevel, _5_AddFoo);
		ActivityBuilder.RenameMap(FooLevel, BarLevel, _6_SaveBar, _7_RenameFooToBar);
		ActivityBuilder.RenameMap(BarLevel, FooLevel, _8_SaveFoo, _9_RenameBarToFoo);
		
		return ActivityBuilder.GetActivities();
	}
}


namespace UE::ConcertSyncTests::TransactionDependencyEdgeCases
{
	enum ETestActivity
	{
		_1_AddFoo,
		_2_CreateActor,
		_3_DeleteFoo,

		// The goal is to test _5_CreateActor does not depend on _2_CreateActor
		_4_AddFoo,
		_5_CreateActor,
		_6_SaveBar,
		_7_RenameFooToBar,

		// The goal is to that _9_CreateActor does not depend on _5_CreateActor
		_8_AddFoo,
		_9_CreateActor,

		ActivityCount
	};
	
	TArray<FActivityID> FillDatabase(FScopedSessionDatabaseWithEndpoint& SessionDatabase);
	
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTransactionDependencyEdgeCaseTests, "VirtualProduction.Concert.History.BuildGraph.TransactionDependencyEdgeCaseTests", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FTransactionDependencyEdgeCaseTests::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncCore;
		
		FScopedSessionDatabaseWithEndpoint SessionDatabase(*this);
		TArray<FActivityID> Activities = FillDatabase(SessionDatabase);
		
		const FActivityDependencyGraph DependencyGraph = BuildDependencyGraphFrom(SessionDatabase);
		UE_LOGF(LogConcert, Log, "%ls tested graph in Graphviz format:\n\n%ls", *GetTestFullName(), *ConcertSyncCore::Graphviz::ExportToGraphviz(DependencyGraph, SessionDatabase));
		TArray<FActivityNodeID> Nodes = ConcertSyncTests::ValidateEachActivityHasNode(*this, Activities, DependencyGraph, ActivityCount, [](const uint32 ActivityType){ return FString::FromInt(ActivityType); });

		// The main take away from this test:
		TestFalse(TEXT("_5_CreateActor does not depend on _2_CreateActor"), DependencyGraph.GetNodeById(Nodes[_2_CreateActor]).AffectsNode(Nodes[_5_CreateActor]));
		TestFalse(TEXT("_9_CreateActor does not depend on _5_CreateActor"), DependencyGraph.GetNodeById(Nodes[_5_CreateActor]).AffectsNode(Nodes[_9_CreateActor]));

		// For completion sake:
		TestTrue(TEXT("_5_CreateActor depends on _4_AddFoo"), DependencyGraph.GetNodeById(Nodes[_5_CreateActor]).DependsOnNode(Nodes[_4_AddFoo]));
		TestEqual(TEXT("_5_CreateActor only depends on _4_AddFoo"), DependencyGraph.GetNodeById(Nodes[_5_CreateActor]).GetDependencies().Num(), 1);
		TestTrue(TEXT("_9_CreateActor depends on _8_AddFoo"), DependencyGraph.GetNodeById(Nodes[_9_CreateActor]).DependsOnNode(Nodes[_8_AddFoo]));
		TestEqual(TEXT("_9_CreateActor only depends on _8_AddFoo"), DependencyGraph.GetNodeById(Nodes[_9_CreateActor]).GetDependencies().Num(), 1);

		return true;
	}

	TArray<FActivityID> FillDatabase(FScopedSessionDatabaseWithEndpoint& SessionDatabase)
	{
		const FName FooLevel = TEXT("/Game/Foo");
		const FName BarLevel = TEXT("/Game/Bar");
		FActivityBuilder ActivityBuilder(SessionDatabase, ActivityCount);

		ActivityBuilder.NewMap(FooLevel, _1_AddFoo);
		ActivityBuilder.CreateActor(FooLevel, _2_CreateActor, TEXT("FirstActor"));
		ActivityBuilder.DeleteMap(FooLevel, _3_DeleteFoo);
		ActivityBuilder.NewMap(FooLevel, _4_AddFoo);
		ActivityBuilder.CreateActor(FooLevel, _5_CreateActor, TEXT("SecondActor"));
		ActivityBuilder.RenameMap(FooLevel, BarLevel, _6_SaveBar, _7_RenameFooToBar);
		ActivityBuilder.NewMap(FooLevel, _8_AddFoo);
		ActivityBuilder.CreateActor(FooLevel, _9_CreateActor, TEXT("ThirdActor"));
		
		return ActivityBuilder.GetActivities();
	}
}
