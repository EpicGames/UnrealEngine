// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryEdition/ActivityDependencyGraph.h"

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "Misc/AutomationTest.h"
#include "HistoryEdition/ActivityGraphIDs.h"
#include "HistoryEdition/ActivityNode.h"

namespace UE::ConcertSyncTests
{
	/** Ensures that the dependency graph API functions as intended, i.e. you add, connect and query nodes. */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertDependencyGraphApiTest, "VirtualProduction.Concert.History.GraphApi.DependencyGraphApi", EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
	bool FConcertDependencyGraphApiTest::RunTest(const FString& Parameters)
	{
		using namespace UE::ConcertSyncCore;
		constexpr int64 FirstActivity = 1;
		constexpr int64 SecondActivity = 20;
		constexpr int64 ThirdActivity = 300;
		
		FActivityDependencyGraph Graph;
		const FActivityNodeID FirstID = Graph.AddActivity(FirstActivity);
		const FActivityNodeID SecondID = Graph.AddActivity(SecondActivity);
		const FActivityNodeID ThirdID = Graph.AddActivity(ThirdActivity);

		const FActivityNode& FirstNode = Graph.GetNodeById(FirstID);
		const FActivityNode& SecondNode = Graph.GetNodeById(SecondID);
		const FActivityNode& ThirdNode = Graph.GetNodeById(ThirdID);

		TestTrue(TEXT("Correct activity in Root"), FirstNode.GetActivityId() == FirstActivity);
		TestTrue(TEXT("Correct activity in LeftNode"), SecondNode.GetActivityId() == SecondActivity);
		TestTrue(TEXT("Correct activity in RightNode"), ThirdNode.GetActivityId() == ThirdActivity);

		const FActivityNodeID Invalid{42342343};
		TestEqual(TEXT("Find activity: First"), Graph.FindNodeByActivity(FirstActivity).Get(Invalid), FirstID);
		TestEqual(TEXT("Find activity: Second"), Graph.FindNodeByActivity(SecondActivity).Get(Invalid), SecondID);
		TestEqual(TEXT("Find activity: Third"), Graph.FindNodeByActivity(ThirdActivity).Get(Invalid), ThirdID);

		const bool bLefttoRoot = Graph.AddDependency(SecondID, { FirstID, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency });
		const bool bRightToRoot = Graph.AddDependency(ThirdID, { FirstID, EActivityDependencyReason::PackageCreation, EDependencyStrength::PossibleDependency });

		TestTrue(TEXT("Added left-to-root"), bLefttoRoot);
		TestTrue(TEXT("Added right-to-root"), bRightToRoot);
		if (!bLefttoRoot || !bRightToRoot)
		{
			return false;
		}

		TestTrue(TEXT("1st knows about 2nd activity"), FirstNode.AffectsAnyActivity() && FirstNode.AffectsNode(SecondID) && FirstNode.AffectsActivity(SecondActivity, Graph));
		TestTrue(TEXT("1st knows about 3rd activilty"), FirstNode.AffectsAnyActivity() && FirstNode.AffectsNode(ThirdID) && FirstNode.AffectsActivity(ThirdActivity, Graph));
		TestTrue(TEXT("2nd knows about 1st"), SecondNode.HasAnyDependency() && SecondNode.DependsOnNode(FirstID, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency) && SecondNode.DependsOnActivity(FirstActivity, Graph, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency));
		TestTrue(TEXT("3rd knows about 1st"), ThirdNode.HasAnyDependency() && ThirdNode.DependsOnNode(FirstID, EActivityDependencyReason::PackageCreation, EDependencyStrength::PossibleDependency) && ThirdNode.DependsOnActivity(FirstActivity, Graph, EActivityDependencyReason::PackageCreation, EDependencyStrength::PossibleDependency));
		return true;
	}

	/** Ensures that the dependency graph API functions as intended, i.e. you add, connect and query nodes. */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDisallowCyclesInGraph, "VirtualProduction.Concert.History.GraphApi.DisallowCyclesInGraph", EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
	bool FDisallowCyclesInGraph::RunTest(const FString& Parameters)
	{
		using namespace UE::ConcertSyncCore;
		constexpr int64 FirstActivity = 1;
		constexpr int64 SecondActivity = 20;
		
		FActivityDependencyGraph Graph;
		const FActivityNodeID FirstNodeID = Graph.AddActivity(FirstActivity);
		const FActivityNodeID SecondNodeID = Graph.AddActivity(SecondActivity);

		const bool bSecondToFirst = Graph.AddDependency(SecondNodeID, { FirstNodeID, EActivityDependencyReason::PackageCreation, EDependencyStrength::HardDependency });
		AddExpectedError(TEXT("Activities can only depend on preceding activities!"), EAutomationExpectedErrorFlags::Contains, 1);
		const bool bFirstToSecond = Graph.AddDependency(FirstNodeID, { SecondNodeID, EActivityDependencyReason::PackageCreation, EDependencyStrength::PossibleDependency });

		TestTrue(TEXT("Later activity depends on early dependency"), bSecondToFirst);
		TestTrue(TEXT("Earlier dependency cannot depend on later dependency"), !bFirstToSecond);
		return true;
	}
}
