// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogGlobal.h"

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "HistoryEdition/DebugDependencyGraph.h"
#include "HistoryEdition/ActivityDependencyGraph.h"
#include "HistoryEdition/DependencyGraphBuilder.h"
#include "HistoryEdition/ActivityGraphIDs.h"
#include "HistoryEdition/HistoryAnalysis.h"
#include "RenameEditAndDeleteMapsFlow.h"
#include "Util/ScopedSessionDatabaseWithEndpoint.h"

namespace UE::ConcertSyncTests::AnalysisTests
{
	/**
	 * Runs through a couple of use cases of unmuting an activity.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBottomUpUseCase, "VirtualProduction.Concert.History.Analysis.BottomUpUseCase", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FBottomUpUseCase::RunTest(const FString& Parameters)
	{
		using namespace RenameEditAndDeleteMapsFlowTest;
		using namespace ConcertSyncCore;
		
		FScopedSessionDatabaseWithEndpoint SessionDatabase(*this);
		const TTestActivityArray<FActivityID> Activities = CreateActivityHistory(SessionDatabase, SessionDatabase.GetEndpoint());
		const FActivityDependencyGraph DependencyGraph = BuildDependencyGraphFrom(SessionDatabase);
		UE_LOGF(LogConcert, Log, "%ls tested graph in Graphviz format:\n\n%ls", *GetTestFullName(), *ConcertSyncCore::Graphviz::ExportToGraphviz(DependencyGraph, SessionDatabase));

		// I highly recommend visualising the above graph to understand the test better
		// Unmute an edit operation that took place after an actor rename
		{
			const FHistoryAnalysisResult UnmuteEditAfterRename = AnalyseActivityDependencies_BottomUp({ Activities[_5_EditActor] }, DependencyGraph);
			TestEqual(TEXT("UnmuteEditAfterRename.HardDependencies.Num() == 2"), UnmuteEditAfterRename.HardDependencies.Num(), 2);
			TestEqual(TEXT("UnmuteEditAfterRename.PossibleDependencies.Num() == 1"), UnmuteEditAfterRename.PossibleDependencies.Num(), 1);
			TestTrue(TEXT("Editing actor depends on creating actor"), UnmuteEditAfterRename.HardDependencies.Contains(Activities[_3_AddActor]));
			TestTrue(TEXT("Creating actor depends on creating map"), UnmuteEditAfterRename.HardDependencies.Contains(Activities[_1_NewPackageFoo]));
			TestTrue(TEXT("Editting possibly depends on renaming"), UnmuteEditAfterRename.PossibleDependencies.Contains(Activities[_4_RenameActor]));
		}

		// Unmute package rename
		{
			const FHistoryAnalysisResult UnmutePackageRename = AnalyseActivityDependencies_BottomUp({ Activities[_7_RenameFooToBar] }, DependencyGraph);
			TestEqual(TEXT("UnmutePackageRename.HardDependencies.Num() == 5"), UnmutePackageRename.HardDependencies.Num(), 5);
			TestEqual(TEXT("UnmutePackageRename.PossibleDependencies.Num() == 0"), UnmutePackageRename.PossibleDependencies.Num(), 0);
			TestTrue(TEXT("_7_RenameFooToBar > _1_NewPackageFoo"), UnmutePackageRename.HardDependencies.Contains(Activities[_1_NewPackageFoo]));
			TestTrue(TEXT("_7_RenameFooToBar > _3_AddActor"), UnmutePackageRename.HardDependencies.Contains(Activities[_3_AddActor]));
			TestTrue(TEXT("_7_RenameFooToBar > _4_RenameActor"), UnmutePackageRename.HardDependencies.Contains(Activities[_4_RenameActor]));
			TestTrue(TEXT("_7_RenameFooToBar > _5_EditActor"), UnmutePackageRename.HardDependencies.Contains(Activities[_5_EditActor]));
			TestTrue(TEXT("_7_RenameFooToBar > _6_SavePackageBar"), UnmutePackageRename.HardDependencies.Contains(Activities[_6_SavePackageBar]));
		}
		
		return true;
	}

	/**
	 * Suppose:
	 *
	 *		R
	 *	   / \
	 *	  A   B
	 *	   \ /
	 *	    L
	 *
	 * The edges L -> A -> R are possible dependencies.
	 * The edges L -> B -> R are hard dependencies.
	 *
	 * The test: unmute L.
	 * We want R to be marked has a hard dependency.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBottomUpDiamond, "VirtualProduction.Concert.History.Analysis.BottomUpDiamond", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FBottomUpDiamond::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncCore;

		constexpr FActivityID RootActivityID = 1;
		constexpr FActivityID AActivityID = 2;
		constexpr FActivityID BActivityID = 3;
		constexpr FActivityID LeafActivityID  = 4;
		
		FActivityDependencyGraph DependencyGraph;
		const FActivityNodeID RootNodeID = DependencyGraph.AddActivity(RootActivityID);
		const FActivityNodeID ANodeID = DependencyGraph.AddActivity(AActivityID);
		const FActivityNodeID BNodeID = DependencyGraph.AddActivity(BActivityID);
		const FActivityNodeID LeafNodeID = DependencyGraph.AddActivity(LeafActivityID);

		// Add the weak dependency first so the algorithm finds it first when iterating
		DependencyGraph.AddDependency(LeafNodeID, FActivityDependencyEdge(ANodeID, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::PossibleDependency));
		DependencyGraph.AddDependency(ANodeID, FActivityDependencyEdge(RootNodeID, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::PossibleDependency));
		DependencyGraph.AddDependency(LeafNodeID, FActivityDependencyEdge(BNodeID, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::HardDependency));
		DependencyGraph.AddDependency(BNodeID, FActivityDependencyEdge(RootNodeID, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::HardDependency));

		const FHistoryAnalysisResult DeleteFooRequirements = AnalyseActivityDependencies_BottomUp({ LeafActivityID }, DependencyGraph);

		TestEqual(TEXT("HardDependencies.Num() == 2"), DeleteFooRequirements.HardDependencies.Num(), 2);
		TestTrue(TEXT("HardDependencies.Contains(B)"), DeleteFooRequirements.HardDependencies.Contains(BActivityID));
		TestTrue(TEXT("HardDependencies.Contains(L)"), DeleteFooRequirements.HardDependencies.Contains(RootActivityID));
		
		TestEqual(TEXT("PossibleDependencies.Num() == 1"), DeleteFooRequirements.PossibleDependencies.Num(), 1);
		TestTrue(TEXT("PossibleDependencies.Contains(A)"), DeleteFooRequirements.PossibleDependencies.Contains(AActivityID));

		return true;
	}
}
