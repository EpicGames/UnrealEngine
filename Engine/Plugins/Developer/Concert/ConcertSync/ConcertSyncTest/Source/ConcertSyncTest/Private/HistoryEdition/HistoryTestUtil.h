// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HistoryEdition/ActivityDependencyGraph.h"
#include "HistoryEdition/ActivityGraphIDs.h"
#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests
{
	inline TArray<ConcertSyncCore::FActivityNodeID> ValidateEachActivityHasNode(
		FAutomationTestBase& Test,
		const TArray<int64>& ActivityMappings,
		const ConcertSyncCore::FActivityDependencyGraph& Graph,
		uint32 ActivityCount,
		TFunctionRef<FString(uint32 ActivityType)> LexToString)
	{
		using namespace ConcertSyncCore;
		
		TArray<FActivityNodeID> ActivityNodes;
		ActivityNodes.SetNumZeroed(ActivityCount);
		for (uint32 ActivityType = 0; ActivityType < ActivityCount; ++ActivityType)
		{
			const int64 ActivityId = ActivityMappings[ActivityType];
			const TOptional<FActivityNodeID> NodeID = Graph.FindNodeByActivity(ActivityId);
			if (!NodeID.IsSet())
			{
				Test.AddError(FString::Printf(TEXT("No node generated for activity type %s"), *LexToString(ActivityType)));
				continue;
			}
			
			if (!NodeID.IsSet())
			{
				Test.AddError(FString::Printf(TEXT("Graph has invalid state. Node ID %lld is invalid for activity type %s"), NodeID->ID, *LexToString(ActivityType)));
				continue;
			}
			
			ActivityNodes[ActivityType] = *NodeID;
		}

		return ActivityNodes;
	}
}
