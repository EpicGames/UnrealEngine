// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogGlobal.h"

#include "HistoryEdition/ActivityDependencyEdge.h"
#include "HistoryEdition/DebugDependencyGraph.h"
#include "HistoryEdition/ActivityDependencyGraph.h"
#include "HistoryEdition/DependencyGraphBuilder.h"
#include "HistoryEdition/ActivityGraphIDs.h"
#include "HistoryEdition/HistoryAnalysis.h"
#include "RenameEditAndDeleteMapsFlow.h"
#include "Util/ScopedSessionDatabaseWithEndpoint.h"

#include "Algo/AllOf.h"

namespace UE::ConcertSyncTests
{
	bool ValidateRequirements(
		const FString& TestBaseName,
		FAutomationTestBase& Test,
		const RenameEditAndDeleteMapsFlowTest::TTestActivityArray<FActivityID> Activities,
		const ConcertSyncCore::FHistoryAnalysisResult& ToValidate = {},
		const TSet<RenameEditAndDeleteMapsFlowTest::ETestActivity>& ExpectedHardDependencies = {},
		const TSet<RenameEditAndDeleteMapsFlowTest::ETestActivity>& ExpectedPossibleDependencies = {}
		)
	{
		using namespace RenameEditAndDeleteMapsFlowTest;

		const bool bHardDependenciesAreCorrect = Algo::AllOf(ExpectedHardDependencies, [&TestBaseName, &Test, &Activities, &ToValidate](ETestActivity Activity)
		{
			const bool bContained = ToValidate.HardDependencies.Contains(Activities[Activity]);
			Test.TestTrue(FString::Printf(TEXT("%s: %s is a hard dependency"), *TestBaseName, *LexToString(Activity)), bContained);
			return bContained;
		});
		Test.TestTrue(FString::Printf(TEXT("%s: Hard dependencies are correct"), *TestBaseName), bHardDependenciesAreCorrect);
		
		const bool bPossibleDependenciesAreCorrect = Algo::AllOf(ExpectedPossibleDependencies, [&TestBaseName, &Test, &Activities, &ToValidate](ETestActivity Activity)
		{
			const bool bContained = ToValidate.PossibleDependencies.Contains(Activities[Activity]);
			Test.TestTrue(FString::Printf(TEXT("%s: %s is a possible dependency"), *TestBaseName, *LexToString(Activity)), bContained);
			return bContained;
		});
		Test.TestTrue(FString::Printf(TEXT("%s: Possible dependencies are correct"), *TestBaseName), bPossibleDependenciesAreCorrect);
		
		const TSet<ETestActivity> ExpectedExcludedActivities = AllActivities().Difference(ExpectedHardDependencies.Union(ExpectedPossibleDependencies));
		const bool bAllOtherActivitiesExcluded = Algo::AllOf(ExpectedExcludedActivities, [&TestBaseName, &Test, &Activities, &ToValidate](ETestActivity Activity)
		{
			const bool bNotContained = !ToValidate.HardDependencies.Contains(Activities[Activity]) && !ToValidate.PossibleDependencies.Contains(Activities[Activity]);
			Test.TestTrue(FString::Printf(TEXT("%s: %s is no dependency"), *TestBaseName, *LexToString(Activity)), bNotContained);
			return bNotContained;
		});
		Test.TestTrue(FString::Printf(TEXT("%s: No unexpected dependencies"), *TestBaseName), bAllOtherActivitiesExcluded);

		return bHardDependenciesAreCorrect && bPossibleDependenciesAreCorrect && bAllOtherActivitiesExcluded;
	}
}

namespace UE::ConcertSyncTests::AnalysisTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTopDownUseCase, "VirtualProduction.Concert.History.Analysis.TopDownUseCase", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FTopDownUseCase::RunTest(const FString& Parameters)
	{
		using namespace RenameEditAndDeleteMapsFlowTest;
		using namespace ConcertSyncCore;
		
		FScopedSessionDatabaseWithEndpoint SessionDatabase(*this);
		const TTestActivityArray<FActivityID> Activities = CreateActivityHistory(SessionDatabase, SessionDatabase.GetEndpoint());
		const FActivityDependencyGraph DependencyGraph = BuildDependencyGraphFrom(SessionDatabase);
		UE_LOGF(LogConcert, Log, "%ls tested graph in Graphviz format:\n\n%ls", *GetTestFullName(), *ConcertSyncCore::Graphviz::ExportToGraphviz(DependencyGraph, SessionDatabase));

		// Delete /Game/Foo > Nearly everything has hard dependency
		{
			const FHistoryAnalysisResult DeleteFooRequirements = AnalyseActivityDependencies_TopDown({ Activities[_1_NewPackageFoo] }, DependencyGraph);
			const TSet<ETestActivity> ExcludedHardActivities { _1_NewPackageFoo };
			const TSet<ETestActivity> HardDependencies = AllActivities().Difference(ExcludedHardActivities);
			const TSet<ETestActivity> PossibleDependencies { };
			const bool bDeleteAllCorrect = ValidateRequirements(TEXT("Delete /Game/Foo"), *this, Activities, DeleteFooRequirements, HardDependencies, PossibleDependencies);
			TestTrue(TEXT("Delete /Game/Foo is correct"), bDeleteAllCorrect);
		}

		// Delete rename transaction > No dependencies
			// because the rename basically just changes AActor::ActorLabel
		{
			const FHistoryAnalysisResult DeleteRenameRequirements = AnalyseActivityDependencies_TopDown({ Activities[_4_RenameActor] }, DependencyGraph);
			TestEqual(TEXT("Delete renaming actor: HardDependencies.Num() == 6"), DeleteRenameRequirements.HardDependencies.Num(), 6);
			TestEqual(TEXT("Delete renaming actor: PossibleDependencies.Num() == 1"), DeleteRenameRequirements.PossibleDependencies.Num(), 1);
			TestTrue(TEXT("Delete renaming actor: Edit activity may depend on deleted activity"), DeleteRenameRequirements.PossibleDependencies.Contains(Activities[_5_EditActor]));
		}

		// Delete actor creation > All transactions operating on actor are hard dependencies
		{
			const FHistoryAnalysisResult DeleteCreateActorRequirements = AnalyseActivityDependencies_TopDown({ Activities[_3_AddActor] }, DependencyGraph);
			TestEqual(TEXT("Delete actor creation: HardDependencies.Num() == 8"), DeleteCreateActorRequirements.HardDependencies.Num(), 8);
			TestEqual(TEXT("Delete actor creation: PossibleDependencies.Num() == 0"), DeleteCreateActorRequirements.PossibleDependencies.Num(), 0);
			TestTrue(TEXT("Delete actor creation: Rename depends on created actor"), DeleteCreateActorRequirements.HardDependencies.Contains(Activities[_4_RenameActor]));
			TestTrue(TEXT("Delete actor creation: Edit depends on created actor"), DeleteCreateActorRequirements.HardDependencies.Contains(Activities[_5_EditActor]));
		}

		// Deleting a rename activity > Rename activity should have a possible dependency to activities that saved the renamed-to package
			// because MU internally creates a Save Package activity before a rename activity (so secretly rename activities is two related activities)
		{
			const FHistoryAnalysisResult DeleteRenameRequirements = AnalyseActivityDependencies_TopDown({ Activities[_7_RenameFooToBar] }, DependencyGraph);

			// Not really what we want to test - but we're doing it for completness
			TestEqual(TEXT("Delete rename package: HardDependencies.Num() == 4"), DeleteRenameRequirements.HardDependencies.Num(), 4);
			TestTrue(TEXT("Delete rename package: Rename has hard dependency creating renamed package"), DeleteRenameRequirements.HardDependencies.Contains(Activities[_8_EditActor]));
			TestTrue(TEXT("Delete rename package: Rename has hard dependency to editing renamed package (1)"), DeleteRenameRequirements.HardDependencies.Contains(Activities[_9_DeleteBar]));
			TestTrue(TEXT("Delete rename package: Rename has hard dependency to editing renamed package (2)"), DeleteRenameRequirements.HardDependencies.Contains(Activities[_10_NewPackageFoo]));
			TestTrue(TEXT("Delete rename package: Rename has hard dependency to editing renamed package (3)"), DeleteRenameRequirements.HardDependencies.Contains(Activities[_11_SavePackageFoo]));

			// Now what we really wanted to test
			TestEqual(TEXT("Delete rename package: PossibleDependencies.Num() == 1"), DeleteRenameRequirements.PossibleDependencies.Num(), 1);
			TestTrue(TEXT("Delete rename package: Rename has soft dependency to saving renamed-to package"), DeleteRenameRequirements.PossibleDependencies.Contains(Activities[_6_SavePackageBar]));
		}

		return true;
	}

	/**
	 * Suppose:
	 *
	 *		R
	 *	   / \
	 *	  A   B
	 *	   \ /
	 *	    L
	 *
	 * The edges L -> A -> R are possible dependencies.
	 * The edges L -> B -> R are hard dependencies.
	 *
	 * The test: delete R.
	 * We want L to be marked has a hard dependency.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTopDownDiamond, "VirtualProduction.Concert.History.Analysis.TopDownDiamond", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FTopDownDiamond::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncCore;

		constexpr FActivityID RootActivityID = 1;
		constexpr FActivityID AActivityID = 2;
		constexpr FActivityID BActivityID = 3;
		constexpr FActivityID LeafActivityID  = 4;
		
		FActivityDependencyGraph DependencyGraph;
		const FActivityNodeID RootNodeID = DependencyGraph.AddActivity(RootActivityID);
		const FActivityNodeID ANodeID = DependencyGraph.AddActivity(AActivityID);
		const FActivityNodeID BNodeID = DependencyGraph.AddActivity(BActivityID);
		const FActivityNodeID LeafNodeID = DependencyGraph.AddActivity(LeafActivityID);

		// Add the weak dependency first so the algorithm finds it first when iterating
		DependencyGraph.AddDependency(LeafNodeID, FActivityDependencyEdge(ANodeID, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::PossibleDependency));
		DependencyGraph.AddDependency(ANodeID, FActivityDependencyEdge(RootNodeID, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::PossibleDependency));
		DependencyGraph.AddDependency(LeafNodeID, FActivityDependencyEdge(BNodeID, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::HardDependency));
		DependencyGraph.AddDependency(BNodeID, FActivityDependencyEdge(RootNodeID, EActivityDependencyReason::EditAfterPreviousPackageEdit, EDependencyStrength::HardDependency));

		const FHistoryAnalysisResult DeleteFooRequirements = AnalyseActivityDependencies_TopDown({ RootActivityID }, DependencyGraph);

		TestEqual(TEXT("HardDependencies.Num() == 2"), DeleteFooRequirements.HardDependencies.Num(), 2);
		TestTrue(TEXT("HardDependencies.Contains(B)"), DeleteFooRequirements.HardDependencies.Contains(BActivityID));
		TestTrue(TEXT("HardDependencies.Contains(L)"), DeleteFooRequirements.HardDependencies.Contains(LeafActivityID));
		
		TestEqual(TEXT("PossibleDependencies.Num() == 1"), DeleteFooRequirements.PossibleDependencies.Num(), 1);
		TestTrue(TEXT("PossibleDependencies.Contains(A)"), DeleteFooRequirements.PossibleDependencies.Contains(AActivityID));

		return true;
	}
}
