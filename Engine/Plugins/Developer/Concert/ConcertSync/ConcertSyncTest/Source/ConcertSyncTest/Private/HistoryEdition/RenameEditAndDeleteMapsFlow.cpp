// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenameEditAndDeleteMapsFlow.h"

#include "ConcertLogGlobal.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"

namespace UE::ConcertSyncTests::RenameEditAndDeleteMapsFlowTest
{
	template<typename T>
	T MakeActivity(const FGuid& EndpointID)
	{
		T Activity;
		Activity.EndpointId = EndpointID;
		return Activity;
	}

	TSet<ETestActivity> AllActivities()
	{
		return {
			_1_NewPackageFoo,
			_2_SavePackageFoo,
			_3_AddActor,
			_4_RenameActor,
			_5_EditActor,
			_6_SavePackageBar,
			_7_RenameFooToBar,
			_8_EditActor,
			_9_DeleteBar,
			_10_NewPackageFoo,
			_11_SavePackageFoo
		};
	}

	FString LexToString(ETestActivity Activity)
	{
		switch (Activity)
		{
		case _1_NewPackageFoo: return TEXT("\"1 New package Foo\"");
		case _2_SavePackageFoo: return TEXT("\"1 Saved package Foo\"");
		case _3_AddActor: return TEXT("\"2 Create actor\"");
		case _4_RenameActor: return TEXT("\"3 Edit actor\"");
		case _5_EditActor: return TEXT("\"4 Edit actor\"");
		case _6_SavePackageBar: return TEXT("\"5 Save package\"");
		case _7_RenameFooToBar: return TEXT("\"5 Rename Foo to Bar\"");
		case _8_EditActor: return TEXT("\"6 Edit actor\"");
		case _9_DeleteBar: return TEXT("\"7 Delete package Bar\"");
		case _10_NewPackageFoo: return TEXT("\"8 Create package Bar\"");
		case _11_SavePackageFoo: return TEXT("\"8 Save package Bar\"");
				
		case ActivityCount:
		default:
			checkNoEntry();
			return FString();
		}
	}

	struct FCreatedStaticMeshActor
	{
		FConcertExportedObject Actor;
		FConcertExportedObject StaticMeshComponent;
	};
	
	FCreatedStaticMeshActor CreateEditedActor(FName OuterLevelPath);

	TTestActivityArray<FActivityID> CreateActivityHistory(FConcertSyncSessionDatabase& SessionDatabase, const FGuid& EndpointID)
	{
		TTestActivityArray<int64> ActivityIDs;
		TTestActivityArray<int64> PackageEventIDs;
		ActivityIDs.SetNumUninitialized(ActivityCount);
		PackageEventIDs.SetNumUninitialized(ActivityCount);
		
		bool bAllSucceeded = true;
		
		// The names of the activities make it into the generated Graphviz graph 
		const FName EditedActorName = TEXT("Actor");
		const FName FooLevel = TEXT("/Game/Foo");
		const FName BarLevel = TEXT("/Game/Bar");
		
		// 1 Create map Foo
		{
			FConcertSyncActivity NewPackage = MakeActivity<FConcertSyncActivity>(EndpointID);
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageInfo.PackageName = FooLevel;
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Added;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			bAllSucceeded &= SessionDatabase.AddPackageActivity(NewPackage, PackageInfo, PackageDataStream, ActivityIDs[_1_NewPackageFoo], PackageEventIDs[_1_NewPackageFoo]);

			FConcertSyncActivity SavePackage = MakeActivity<FConcertSyncActivity>(EndpointID);
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Saved;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			bAllSucceeded &= SessionDatabase.AddPackageActivity(SavePackage, PackageInfo, PackageDataStream, ActivityIDs[_2_SavePackageFoo], PackageEventIDs[_2_SavePackageFoo]);
		}

		// 2 Add actor A
		{
			FConcertSyncTransactionActivity CreateActor = MakeActivity<FConcertSyncTransactionActivity>(EndpointID);
			CreateActor.EventData.Transaction.TransactionId = FGuid::NewGuid();
			CreateActor.EventData.Transaction.OperationId = FGuid::NewGuid();
			FCreatedStaticMeshActor NewActorData = CreateEditedActor(FooLevel);
			NewActorData.Actor.ObjectData.bAllowCreate = true;
			NewActorData.StaticMeshComponent.ObjectData.bAllowCreate = true;
			CreateActor.EventData.Transaction.ExportedObjects = { NewActorData.StaticMeshComponent, NewActorData.Actor };
			CreateActor.EventData.Transaction.ModifiedPackages = { FooLevel };
			SessionDatabase.GetTransactionMaxEventId(CreateActor.EventId);
			bAllSucceeded &= SessionDatabase.AddTransactionActivity(CreateActor, ActivityIDs[_3_AddActor], PackageEventIDs[_3_AddActor]);
		}

		// 3 Rename actor A
		{
			FConcertSyncTransactionActivity EditActor = MakeActivity<FConcertSyncTransactionActivity>(EndpointID);
			EditActor.EventData.Transaction.TransactionId = FGuid::NewGuid();
			EditActor.EventData.Transaction.OperationId = FGuid::NewGuid();
			
			FCreatedStaticMeshActor NewActorData = CreateEditedActor(FooLevel);
			NewActorData.Actor.PropertyDatas = { { TEXT("ActorLabel"), {} } };
			EditActor.EventData.Transaction.ExportedObjects = { NewActorData.Actor };
			EditActor.EventData.Transaction.ModifiedPackages = { FooLevel };
			SessionDatabase.GetTransactionMaxEventId(EditActor.EventId);
			bAllSucceeded &= SessionDatabase.AddTransactionActivity(EditActor, ActivityIDs[_4_RenameActor], PackageEventIDs[_4_RenameActor]);
		}

		// 4 Edit actor A
		{
			FConcertSyncTransactionActivity EditActor = MakeActivity<FConcertSyncTransactionActivity>(EndpointID);
			EditActor.EventData.Transaction.TransactionId = FGuid::NewGuid();
			EditActor.EventData.Transaction.OperationId = FGuid::NewGuid();
			FCreatedStaticMeshActor NewActorData = CreateEditedActor(FooLevel);
			NewActorData.StaticMeshComponent.PropertyDatas = { { TEXT("RelativeLocation"), {} } };
			EditActor.EventData.Transaction.ExportedObjects = { NewActorData.StaticMeshComponent };
			EditActor.EventData.Transaction.ModifiedPackages = { FooLevel };
			SessionDatabase.GetTransactionMaxEventId(EditActor.EventId);
			bAllSucceeded &= SessionDatabase.AddTransactionActivity(EditActor, ActivityIDs[_5_EditActor], PackageEventIDs[_5_EditActor]);
		}

		// 5 Rename map to Bar
		{
			FConcertSyncActivity SaveBarPackage = MakeActivity<FConcertSyncActivity>(EndpointID);
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageInfo.PackageName = BarLevel;
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Saved;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			bAllSucceeded &= SessionDatabase.AddPackageActivity(SaveBarPackage, PackageInfo, PackageDataStream, ActivityIDs[_6_SavePackageBar], PackageEventIDs[_6_SavePackageBar]);

			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Renamed;
			PackageInfo.PackageName = FooLevel;
			PackageInfo.NewPackageName = BarLevel;
			bAllSucceeded &= SessionDatabase.AddPackageActivity(SaveBarPackage, PackageInfo, PackageDataStream, ActivityIDs[_7_RenameFooToBar], PackageEventIDs[_7_RenameFooToBar]);
		}

		// 6 Edit actor A
		{
			FConcertSyncTransactionActivity EditActor = MakeActivity<FConcertSyncTransactionActivity>(EndpointID);
			EditActor.EventData.Transaction.TransactionId = FGuid::NewGuid();
			EditActor.EventData.Transaction.OperationId = FGuid::NewGuid();
			FCreatedStaticMeshActor NewActorData = CreateEditedActor(BarLevel);
			NewActorData.StaticMeshComponent.PropertyDatas = { { TEXT("RelativeLocation"), {} } };
			EditActor.EventData.Transaction.ExportedObjects = { NewActorData.StaticMeshComponent };
			EditActor.EventData.Transaction.ModifiedPackages = { BarLevel };
			SessionDatabase.GetTransactionMaxEventId(EditActor.EventId);
			bAllSucceeded &= SessionDatabase.AddTransactionActivity(EditActor, ActivityIDs[_8_EditActor], PackageEventIDs[_8_EditActor]);
		}

		// 7 Delete map Bar
		{
			FConcertSyncActivity SaveBarPackage = MakeActivity<FConcertSyncActivity>(EndpointID);
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageInfo.PackageName = BarLevel;
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Deleted;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			bAllSucceeded &= SessionDatabase.AddPackageActivity(SaveBarPackage, PackageInfo, PackageDataStream, ActivityIDs[_9_DeleteBar], PackageEventIDs[_9_DeleteBar]);
		}
		
		// 8 Create map Bar
		{
			FConcertSyncActivity NewPackage = MakeActivity<FConcertSyncActivity>(EndpointID);
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageInfo.PackageName = TEXT("/Game/Bar");
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Added;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			bAllSucceeded &= SessionDatabase.AddPackageActivity(NewPackage, PackageInfo, PackageDataStream, ActivityIDs[_10_NewPackageFoo], PackageEventIDs[_10_NewPackageFoo]);

			FConcertSyncActivity SavePackage = MakeActivity<FConcertSyncActivity>(EndpointID);
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Saved;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			bAllSucceeded &= SessionDatabase.AddPackageActivity(SavePackage, PackageInfo, PackageDataStream, ActivityIDs[_11_SavePackageFoo], PackageEventIDs[_11_SavePackageFoo]);
		}

		if (!bAllSucceeded)
		{
			UE_LOGF(LogConcert, Error, "Something went wrong creating the activities. Test result may be wrong.")
		}
		return ActivityIDs;
	}

	FCreatedStaticMeshActor CreateEditedActor(FName OuterLevelPath)
	{
		FCreatedStaticMeshActor Result;

		Result.Actor.ObjectId.ObjectName = TEXT("StaticMeshActor0");
		Result.Actor.ObjectId.ObjectPackageName = OuterLevelPath;
		Result.Actor.ObjectId.ObjectOuterPathName = *FString::Printf(TEXT("%s:PersistentLevel"), *OuterLevelPath.ToString());
		Result.Actor.ObjectId.ObjectClassPathName = TEXT("/Script/Engine.StaticMeshActor");
		
		Result.StaticMeshComponent.ObjectId.ObjectName = TEXT("StaticMeshComponent0");
		Result.StaticMeshComponent.ObjectId.ObjectPackageName = OuterLevelPath;
		Result.StaticMeshComponent.ObjectId.ObjectOuterPathName = *FString::Printf(TEXT("%s:PersistentLevel.%s"), *OuterLevelPath.ToString(), *Result.Actor.ObjectId.ObjectName.ToString());
		Result.StaticMeshComponent.ObjectId.ObjectClassPathName = TEXT("/Script/Engine.StaticMeshComponent");
		
		return Result;
	}
}

