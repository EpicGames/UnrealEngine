// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"

class FConcertSyncSessionDatabase;

namespace UE::ConcertSyncTests::RenameEditAndDeleteMapsFlowTest
{
	enum ETestActivity
	{
		_1_NewPackageFoo,
		_2_SavePackageFoo,
		_3_AddActor,
		_4_RenameActor,
		_5_EditActor,
		_6_SavePackageBar,
		_7_RenameFooToBar,
		_8_EditActor,
		_9_DeleteBar,
		_10_NewPackageFoo,
		_11_SavePackageFoo,
		
		ActivityCount
	};
	TSet<ETestActivity> AllActivities();
	FString LexToString(ETestActivity Activity);
	
	/** An array where every entry of ETestActivity is a valid index. */
	template<typename T>
	using TTestActivityArray = TArray<T, TInlineAllocator<ActivityCount>>;

	/**
	 * Creates a session history which resembles the following sequence of user actions:
	 *	1 Create map Foo
	 *	2 Add actor A
	 *	3 Edit actor A
	 *	4 Edit actor A
	 *	5 Rename map to Bar
	 *	6 Edit actor A
	 *	7 Delete map Bar
	 *	8 Create map Bar
	 *
	 *	@return An array where every ETestActivity entry is the index to the activity ID added to SessionDatabase
	 */
	TTestActivityArray<FActivityID> CreateActivityHistory(FConcertSyncSessionDatabase& SessionDatabase, const FGuid& EndpointID);
}
