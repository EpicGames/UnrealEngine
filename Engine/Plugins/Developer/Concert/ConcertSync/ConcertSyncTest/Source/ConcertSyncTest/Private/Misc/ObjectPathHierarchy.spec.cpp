// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/ObjectPathHierarchy.h"
#include "Misc/ObjectPathUtils.h"

namespace UE::ConcertSyncTests
{
	/** Tests functions in ObjectPathUtils.h */
	BEGIN_DEFINE_SPEC(FObjectPathHierarchySpec, "VirtualProduction.Concert.Components.ObjectPathHierarchy", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		ConcertSyncCore::FObjectPathHierarchy Hierarchy;
	END_DEFINE_SPEC(FObjectPathHierarchySpec);

	void FObjectPathHierarchySpec::Define()
	{
		AfterEach([this]() { Hierarchy.Clear(); });
		
		Describe("Explicit hierarchy", [this]()
		{
			BeforeEach([this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0") });
			});

			It("IsInHierarchy", [this]()
			{
				const auto TestIsInHierarchy = [this](const TCHAR* Path, ConcertSyncCore::EHierarchyObjectType Expected = ConcertSyncCore::EHierarchyObjectType::Explicit)
				{
					const TOptional<ConcertSyncCore::EHierarchyObjectType> State = Hierarchy.IsInHierarchy(FSoftObjectPath{ Path });
					TestTrue(FString::Printf(TEXT("Is in hierarchy: %s"), Path), State.IsSet() && *State == Expected);
				};
				const auto TestIsNotInHierarchy = [this](const TCHAR* Path)
				{
					TestFalse(FString::Printf(TEXT("Not in hierarchy: %s"), Path), Hierarchy.IsInHierarchy(FSoftObjectPath{ Path }).IsSet());
				};

				TestIsInHierarchy(TEXT("/Game/Maps.Map"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere") });
				TestIsInHierarchy(TEXT("/Game/Maps.Map"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1"));
				// Actual change:
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere"), ConcertSyncCore::EHierarchyObjectType::Implicit);
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0") });
				TestIsInHierarchy(TEXT("/Game/Maps.Map"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0"));
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1"));
				// Actual change:
				TestIsNotInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere"));
				TestIsNotInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));
			});

			It("HasChildren", [this]()
			{
				const auto TestHasChildren = [this](const TCHAR* Path)
				{
					TestTrue(FString::Printf(TEXT("Has children: %s"), Path), Hierarchy.HasChildren(FSoftObjectPath{ Path }));
				};
				const auto TestHasNoChildren = [this](const TCHAR* Path)
				{
					TestFalse(FString::Printf(TEXT("Has no children: %s"), Path), Hierarchy.HasChildren(FSoftObjectPath{ Path }));
				};

				TestHasChildren(TEXT("/Game/Maps.Map"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere") });
				TestHasChildren(TEXT("/Game/Maps.Map"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1"));
				// Actual change:
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0") });
				TestHasChildren(TEXT("/Game/Maps.Map"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1"));
				// Actual change:
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));
			});
			
			It("IsAssetInHierarchy", [this]()
			{
				TestTrue(TEXT("IsAssetInHierarchy(Map)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map")}));
				
				TestFalse(TEXT("IsAssetInHierarchy(PersistentLevel)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel")}));
				TestFalse(TEXT("IsAssetInHierarchy(Cube)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube")}));
				TestFalse(TEXT("IsAssetInHierarchy(StaticMeshComponent0)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")}));
				TestFalse(TEXT("IsAssetInHierarchy(StaticMeshComponent1)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")}));
				TestFalse(TEXT("IsAssetInHierarchy(Cube)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere")}));
				TestFalse(TEXT("IsAssetInHierarchy(StaticMeshComponent0)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")}));
				
				TestFalse(TEXT("IsAssetInHierarchy(null)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ }));
				TestFalse(TEXT("IsAssetInHierarchy(OtherMap)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.OtherMap")}));
				TestFalse(TEXT("IsAssetInHierarchy(OtherMap's StaticMeshComponent0)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.OtherMap:PersistentLevel.Sphere.StaticMeshComponent0")}));
			});
			
			It("TraverseTopToBottom (all)", [this]()
			{
				TArray<FSoftObjectPath> Visited;
				Hierarchy.TraverseTopToBottom([this, &Visited](const ConcertSyncCore::FChildRelation& Relation)
				{
					Visited.AddUnique(Relation.Parent.Object);
					Visited.AddUnique(Relation.Child.Object);
					
					const TOptional<FSoftObjectPath> ExpectedParent = ConcertSyncCore::GetOuterPath(Relation.Child.Object);
					TestTrue(TEXT("Is direct child"), ExpectedParent && *ExpectedParent == Relation.Parent.Object);
					
					return ConcertSyncCore::ETreeTraversalBehavior::Continue;
				});

				// Any of the following orders is correct
				TArray<FSoftObjectPath> PossibleOrder1
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
				};
				TArray<FSoftObjectPath> PossibleOrder2
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
				};
				TArray<FSoftObjectPath> PossibleOrder3
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")),
				};
				TArray<FSoftObjectPath> PossibleOrder4
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")),
				};

				const bool bIsOneOfExpectedOrders = Visited == PossibleOrder1 || Visited == PossibleOrder2 || Visited == PossibleOrder3 || Visited == PossibleOrder4;
				TestTrue(TEXT("Is correct order"), bIsOneOfExpectedOrders);
			});

			It("TraverseTopToBottom (SkipSubtree)", [this]()
			{
				TArray<FSoftObjectPath> Visited;
				Hierarchy.TraverseTopToBottom([this, &Visited](const ConcertSyncCore::FChildRelation& Relation)
				{
					Visited.AddUnique(Relation.Parent.Object);
					Visited.AddUnique(Relation.Child.Object);
					
					const TOptional<FSoftObjectPath> ExpectedParent = ConcertSyncCore::GetOuterPath(Relation.Child.Object);
					TestTrue(TEXT("Is direct child"), ExpectedParent && *ExpectedParent == Relation.Parent.Object);
					
					return Relation.Child.Object == FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube"))
						? ConcertSyncCore::ETreeTraversalBehavior::SkipSubtree
						: ConcertSyncCore::ETreeTraversalBehavior::Continue;
				});

				// Any of the following orders is correct
				TArray<FSoftObjectPath> PossibleOrder1
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"))
				};
				TArray<FSoftObjectPath> PossibleOrder2
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube"))
				};

				const bool bIsOneOfExpectedOrders = Visited == PossibleOrder1 || Visited == PossibleOrder2;
				TestTrue(TEXT("Is correct order"), bIsOneOfExpectedOrders);
			});
			
			It("TraverseTopToBottom (/Game/Maps.Map:PersistentLevel.Sphere)", [this]()
			{
				bool bVisited = false;
				Hierarchy.TraverseTopToBottom([this, &bVisited](const ConcertSyncCore::FChildRelation& Relation)
				{
					if (bVisited)
					{
						AddError(TEXT("Expected 1 invocation"));
						return ConcertSyncCore::ETreeTraversalBehavior::Break;
					}

					bVisited = true;
					TestEqual(TEXT("Parent"), Relation.Parent.Object.ToString(), TEXT("/Game/Maps.Map:PersistentLevel.Sphere"));
					TestEqual(TEXT("Child"), Relation.Child.Object.ToString(), TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));
					TestEqual(TEXT("Parent type"), Relation.Parent.Type, ConcertSyncCore::EHierarchyObjectType::Explicit);
					TestEqual(TEXT("Child type"), Relation.Child.Type, ConcertSyncCore::EHierarchyObjectType::Explicit);
					
					return ConcertSyncCore::ETreeTraversalBehavior::Continue;
				}, FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere") });
				TestTrue(TEXT("Visited"), bVisited);
			});

			It("TraverseBottomToTop (all)", [this]()
			{
				TArray<FSoftObjectPath> Visited;
				int32 NumRootEncountered = 0;
				Hierarchy.TraverseBottomToTop([this, &Visited, &NumRootEncountered](const ConcertSyncCore::FChildRelation& Relation)
				{
					Visited.AddUnique(Relation.Child.Object);
					
					const TOptional<FSoftObjectPath> ExpectedParent = ConcertSyncCore::GetOuterPath(Relation.Child.Object);
					TestTrue(TEXT("Is direct child"), ExpectedParent && *ExpectedParent == Relation.Parent.Object);

					NumRootEncountered += Relation.Parent.Object == FSoftObjectPath(TEXT("/Game/Maps.Map"));
					return EBreakBehavior::Continue;
				});

				// Any of the following orders is correct
				TArray<FSoftObjectPath> PossibleOrder1
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
				};
				TArray<FSoftObjectPath> PossibleOrder2
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
				};
				TArray<FSoftObjectPath> PossibleOrder3
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
				};
				TArray<FSoftObjectPath> PossibleOrder4
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent1")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Cube")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
				};

				TestEqual(TEXT("Visited /Game/Maps.Map once"), NumRootEncountered, 1);
				Visited.Add(FSoftObjectPath(TEXT("/Game/Maps.Map")));
				
				const bool bIsOneOfExpectedOrders = Visited == PossibleOrder1 || Visited == PossibleOrder2 || Visited == PossibleOrder3 || Visited == PossibleOrder4;
				TestTrue(TEXT("Is correct order"), bIsOneOfExpectedOrders);
			});

			It("TraverseBottomToTop (/Game/Maps.Map:PersistentLevel.Sphere)", [this]()
			{
				bool bVisited = false;
				Hierarchy.TraverseBottomToTop([this, &bVisited](const ConcertSyncCore::FChildRelation& Relation)
				{
					if (bVisited)
					{
						AddError(TEXT("Expected 1 invocation"));
						return EBreakBehavior::Break;
					}

					bVisited = true;
					TestEqual(TEXT("Parent"), Relation.Parent.Object.ToString(), TEXT("/Game/Maps.Map:PersistentLevel.Sphere"));
					TestEqual(TEXT("Child"), Relation.Child.Object.ToString(), TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));
					TestEqual(TEXT("Parent type"), Relation.Parent.Type, ConcertSyncCore::EHierarchyObjectType::Explicit);
					TestEqual(TEXT("Child type"), Relation.Child.Type, ConcertSyncCore::EHierarchyObjectType::Explicit);
					
					return EBreakBehavior::Continue;
				}, FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere") });
				TestTrue(TEXT("Visited"), bVisited);
			});
		});

		Describe("Implicit hierarchy", [this]()
		{
			BeforeEach([this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0.Subobject") });
			});

			It("IsInHierarchy", [this]()
			{
				using namespace ConcertSyncCore;
				const auto TestIsInHierarchy = [this](const TCHAR* Path, EHierarchyObjectType Expected )
				{
					const TOptional<EHierarchyObjectType> State = Hierarchy.IsInHierarchy(FSoftObjectPath{ Path });
					TestTrue(FString::Printf(TEXT("Is in hierarchy: %s"), Path), State.IsSet() && *State == Expected);
				};

				TestIsInHierarchy(TEXT("/Game/Maps.Map"), EHierarchyObjectType::Implicit);
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel"), EHierarchyObjectType::Implicit);
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere"), EHierarchyObjectType::Explicit);
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"), EHierarchyObjectType::Implicit);
				TestIsInHierarchy(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0.Subobject"), EHierarchyObjectType::Explicit);
			});
			
			It("HasChildren", [this]()
			{
				using namespace ConcertSyncCore;
				const auto TestHasChildren = [this](const TCHAR* Path)
				{
					TestTrue(FString::Printf(TEXT("Has children: %s"), Path), Hierarchy.HasChildren(FSoftObjectPath{ Path }));
				};
				const auto TestHasNoChildren = [this](const TCHAR* Path)
				{
					TestFalse(FString::Printf(TEXT("Has no children: %s"), Path), Hierarchy.HasChildren(FSoftObjectPath{ Path }));
				};

				TestHasChildren(TEXT("/Game/Maps.Map"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere"));
				TestHasChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0"));
				TestHasNoChildren(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0.Subobject"));
			});

			It("IsAssetInHierarchy", [this]()
			{
				TestTrue(TEXT("IsAssetInHierarchy(Map)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map")}));
				
				TestFalse(TEXT("IsAssetInHierarchy(PersistentLevel)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel")}));
				TestFalse(TEXT("IsAssetInHierarchy(Cube)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere")}));
				TestFalse(TEXT("IsAssetInHierarchy(StaticMeshComponent0)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")}));
				TestFalse(TEXT("IsAssetInHierarchy(StaticMeshComponent0)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0.Subobject")}));
				
				TestFalse(TEXT("IsAssetInHierarchy(null)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ }));
				TestFalse(TEXT("IsAssetInHierarchy(OtherMap)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.OtherMap")}));
				TestFalse(TEXT("IsAssetInHierarchy(OtherMap's StaticMeshComponent0)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.OtherMap:PersistentLevel.Sphere.StaticMeshComponent0")}));
			});

			It("TraverseTopToBottom (all)", [this]()
			{
				TArray<FSoftObjectPath> Visited;
				Hierarchy.TraverseTopToBottom([this, &Visited](const ConcertSyncCore::FChildRelation& Relation)
				{
					Visited.AddUnique(Relation.Parent.Object);
					Visited.AddUnique(Relation.Child.Object);
					
					const TOptional<FSoftObjectPath> ExpectedParent = ConcertSyncCore::GetOuterPath(Relation.Child.Object);
					TestTrue(TEXT("Is direct child"), ExpectedParent && *ExpectedParent == Relation.Parent.Object);
					
					return ConcertSyncCore::ETreeTraversalBehavior::Continue;
				});
				
				TArray<FSoftObjectPath> ExpectedOrder
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0.Subobject"))
				};
				TestEqual(TEXT("Visted in expected order"), Visited, ExpectedOrder);
			});
			
			It("TraverseBottomToTop (all)", [this]()
			{
				TArray<FSoftObjectPath> Visited;
				Hierarchy.TraverseBottomToTop([this, &Visited](const ConcertSyncCore::FChildRelation& Relation)
				{
					Visited.AddUnique(Relation.Child.Object);
					Visited.AddUnique(Relation.Parent.Object);
					
					const TOptional<FSoftObjectPath> ExpectedParent = ConcertSyncCore::GetOuterPath(Relation.Child.Object);
					TestTrue(TEXT("Is direct child"), ExpectedParent && *ExpectedParent == Relation.Parent.Object);
					
					return EBreakBehavior::Continue;
				});

				TArray<FSoftObjectPath> ExpectedOrder
				{
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0.Subobject")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere.StaticMeshComponent0")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel.Sphere")),
					FSoftObjectPath(TEXT("/Game/Maps.Map:PersistentLevel")),
					FSoftObjectPath(TEXT("/Game/Maps.Map"))
				};
				TestEqual(TEXT("Visted in expected order"), Visited, ExpectedOrder);
			});
		});

		Describe("Add & remove all (world)", [this]()
		{
			AfterEach([this]()
			{
				TestTrue("IsEmpty()", Hierarchy.IsEmpty());
				TestFalse("HasChildren(PersistentLevel)", Hierarchy.HasChildren(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") }));
				TestFalse("HasChildren(Cube)", Hierarchy.HasChildren(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") }));
				TestFalse("IsInHierarchy(PersistentLevel)", Hierarchy.IsInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") }).IsSet());
				TestFalse("IsInHierarchy(Cube)", Hierarchy.IsInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") }).IsSet());
			});

			It("Add cube, remove cube.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
			});

			// Test various orders in which may be added & removed.

			It("Add level, add cube, remove level, remove cube.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
			});
			It("Add level, add cube, remove cube, remove level.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });
			});

			It("Add cube, add level, remove level, remove cube.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
			});
			It("Add cube, add level, remove cube, remove level.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel") });
			});
		});

		// The difference to the world tests above is that we're directly removing an object that is placed in FObjectPathHierarchy::AssetNodes
		Describe("Add & remove all (asset)", [this]()
		{
			AfterEach([this]()
			{
				TestTrue("IsEmpty()", Hierarchy.IsEmpty());
				TestFalse("HasChildren(Root)", Hierarchy.HasChildren(FSoftObjectPath{ TEXT("/Engine/Transient.Root") }));
				TestFalse("HasChildren(Subobject)", Hierarchy.HasChildren(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") }));
				TestFalse("IsInHierarchy(Root)", Hierarchy.IsInHierarchy(FSoftObjectPath{ TEXT("/Engine/Transient.Root") }).IsSet());
				TestFalse("IsInHierarchy(Subobject)", Hierarchy.IsInHierarchy(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") }).IsSet());
			});

			It("Add subobject, remove subobject.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });
			});

			// Test various orders in which may be added & removed.

			It("Add root, add subobject, remove root, remove subobject.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });
			});
			It("Add root, add subobject, remove subobject, remove root.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root") });
			});

			It("Add subobject, add root, remove root, remove subobject.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root") });

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });
			});
			It("Add subobject, add root, remove subobject, remove root.", [this]()
			{
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });
				Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root") });

				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root:Subobject") });
				Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Engine/Transient.Root") });
			});
		});

		It("Retain hierarchy after removal", [this]()
		{
			Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.Foo") });
			Hierarchy.AddObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.Bar") });
			// We'll proceed to test that the rest of the hierarchy remains intact.
			Hierarchy.RemoveObject(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.Foo")});

			// There used to be a bug where FObjectPathHierarchy::AssetNodes would be emptied incorrectly and FObjectPathHierarchy::CachedNodes left dangling. Check that case specifically with IsAssetInHierarchy:
			TestTrue(TEXT("IsAsset(Map)"), Hierarchy.IsAssetInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map")}));
			
			TestTrue(TEXT("IsInHierarchy(Map)"), Hierarchy.IsInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map")}).IsSet());
			TestTrue(TEXT("IsInHierarchy(Cube)"), Hierarchy.IsInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel")}).IsSet());
			TestTrue(TEXT("IsInHierarchy(Cube)"), Hierarchy.IsInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube")}).IsSet());
			TestTrue(TEXT("IsInHierarchy(Bar)"), Hierarchy.IsInHierarchy(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube.Bar")}).IsSet());
			TestTrue(TEXT("HasChildren(Cube)"), Hierarchy.HasChildren(FSoftObjectPath{ TEXT("/Game/Maps.Map:PersistentLevel.Cube")}));
		});
	}
}
