// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/ObjectPathOuterIterator.h"
#include "Misc/ObjectPathUtils.h"
#include "Misc/ObjectUtils.h"

namespace UE::ConcertSyncTests
{
	/** Tests functions in ObjectPathUtils.h */
	BEGIN_DEFINE_SPEC(FObjectPathUtilsSpec, "VirtualProduction.Concert.Components.ObjectPathUtils", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	END_DEFINE_SPEC(FObjectPathUtilsSpec);

	void FObjectPathUtilsSpec::Define()
	{
		It("FObjectPathOuterIterator", [this]()
		{
			int32 NumberOfInvocations = 0;
			const FSoftObjectPath ComponentPath{ TEXT("/Game/Map.Map:PersistentLevel.Actor.Component") };
			for (ConcertSyncCore::FObjectPathOuterIterator It(ComponentPath); It; ++It)
			{
				++NumberOfInvocations;
				
				const FString PathString = It->ToString();
				switch (NumberOfInvocations)
				{
				case 1: TestEqual(TEXT("Equal to /Game/Map.Map:PersistentLevel.Actor"), PathString, TEXT("/Game/Map.Map:PersistentLevel.Actor")); break;
				case 2: TestEqual(TEXT("Equal to /Game/Map.Map:PersistentLevel"), PathString, TEXT("/Game/Map.Map:PersistentLevel")); break;
				case 3: TestEqual(TEXT("Equal to /Game/Map.Map"), PathString, TEXT("/Game/Map.Map")); break;
				default: AddError(TEXT("Too many invocations"));
				}
			}
			TestEqual(TEXT("Invoked exactly 3 times"), NumberOfInvocations, 3);

			for (ConcertSyncCore::FObjectPathOuterIterator It(FSoftObjectPath{ TEXT("/Game/Map.Map") }); It; ++It)
			{
				AddError(TEXT("Assets do not have any outers"));
			}
			for (ConcertSyncCore::FObjectPathOuterIterator It(FSoftObjectPath{}); It; ++It)
			{
				AddError(TEXT("Null iteration"));
			}
		});

		It("GetOuterPath", [this]()
		{
			const TOptional<FSoftObjectPath> ActorPath = ConcertSyncCore::GetOuterPath(FSoftObjectPath{ TEXT("/Game/Map.Map:PersistentLevel.Actor.Component") });
			TestEqual(
				TEXT("/Game/Map.Map:PersistentLevel.Actor.Component"),
				ActorPath ? ActorPath->ToString() : TEXT(""),
				TEXT("/Game/Map.Map:PersistentLevel.Actor")
				);
			
			const TOptional<FSoftObjectPath> PersistentLevelPath = ConcertSyncCore::GetOuterPath(FSoftObjectPath{ TEXT("/Game/Map.Map:PersistentLevel.Actor") });
			TestEqual(
				TEXT("/Game/Map.Map:PersistentLevel.Actor"),
				PersistentLevelPath ? PersistentLevelPath->ToString() : TEXT(""),
				TEXT("/Game/Map.Map:PersistentLevel")
				);
			
			const TOptional<FSoftObjectPath> PackagePath = ConcertSyncCore::GetOuterPath(FSoftObjectPath{ TEXT("/Game/Map.Map:PersistentLevel") });
			TestEqual(
				TEXT("/Game/Map.Map:PersistentLevel"),
				PackagePath ? PackagePath->ToString() : TEXT(""),
				TEXT("/Game/Map.Map")
				);

			const TOptional<FSoftObjectPath> NoPath = ConcertSyncCore::GetOuterPath(FSoftObjectPath{ TEXT("/Game/Map.Map") });
			TestFalse(TEXT("/Game/Map.Map"), NoPath.IsSet());
			const TOptional<FSoftObjectPath> NullPath = ConcertSyncCore::GetOuterPath(nullptr);
			TestFalse(TEXT("Null"), NullPath.IsSet());
		});

		Describe("ReplaceActorOf", [this]
		{
			Describe("Positive (valid args)", [this]
			{
				It("With subobject in OldPath", [this]
				{
					const FSoftObjectPath OldPath(TEXT("/Game/Map.Map:PersistentLevel.OldActor.Subobject"));
					const FSoftObjectPath NewPath(TEXT("/Game/Map.Map:PersistentLevel.NewActor"));
					const TOptional<FSoftObjectPath> Replacement = ConcertSyncCore::ReplaceActorInPath(OldPath, NewPath);
					TestTrue(TEXT("Equals"), Replacement && *Replacement == FSoftObjectPath(TEXT("/Game/Map.Map:PersistentLevel.NewActor.Subobject")));
				});

				It("Only replace actor", [this]
				{
					const FSoftObjectPath OldSubobject(TEXT("/Game/Map.Map:PersistentLevel.Actor"));
					const FSoftObjectPath NewPath(TEXT("/Game/Map.Map:PersistentLevel.NewActor"));
					const TOptional<FSoftObjectPath> Replacement = ConcertSyncCore::ReplaceActorInPath(OldSubobject, NewPath);
					TestTrue(TEXT("Equals"), Replacement && *Replacement == FSoftObjectPath(TEXT("/Game/Map.Map:PersistentLevel.NewActor")));
				});

				It("Only replace path", [this]
				{
					const FSoftObjectPath OldSubobject(TEXT("/Game/OldMap.OldMap:PersistentLevel.Actor"));
					const FSoftObjectPath NewPath(TEXT("/Game/NewMap.NewMap:PersistentLevel.Actor"));
					const TOptional<FSoftObjectPath> Replacement = ConcertSyncCore::ReplaceActorInPath(OldSubobject, NewPath);
					TestTrue(TEXT("Equals"), Replacement && *Replacement == FSoftObjectPath(TEXT("/Game/NewMap.NewMap:PersistentLevel.Actor")));
				});

				It("Actor called \"PersistentLevel\"", [this]
				{
					const FSoftObjectPath OldSubobject(TEXT("/Game/OldMap.OldMap:PersistentLevel.PersistentLevel.Subobject"));
					const FSoftObjectPath NewPath(TEXT("/Game/NewMap.NewMap:PersistentLevel.PersistentLevel"));
					const TOptional<FSoftObjectPath> Replacement = ConcertSyncCore::ReplaceActorInPath(OldSubobject, NewPath);
					TestTrue(TEXT("Equals"), Replacement && *Replacement == FSoftObjectPath(TEXT("/Game/NewMap.NewMap:PersistentLevel.PersistentLevel.Subobject")));
				});
			});
			
			Describe("Negative (invalid args)", [this]
			{
				It("OldPath has no actor", [this]
				{
					const FSoftObjectPath OldSubobject(TEXT("/Game/Map.Map"));
					const FSoftObjectPath NewActor(TEXT("/Game/Map.Map:PersistentLevel.Actor.Subobject"));
					const TOptional<FSoftObjectPath> Replacement = ConcertSyncCore::ReplaceActorInPath(OldSubobject, NewActor);
					TestFalse(TEXT("Invalid"), Replacement.IsSet());
				});

				It("NewPath has no actor", [this]
				{
					const FSoftObjectPath OldSubobject(TEXT("/Game/Map.Map:PersistentLevel.Actor.Subobject"));
					const FSoftObjectPath NewActor(TEXT("/Game/Map.Map"));
					const TOptional<FSoftObjectPath> Replacement = ConcertSyncCore::ReplaceActorInPath(OldSubobject, NewActor);
					TestFalse(TEXT("Invalid"), Replacement.IsSet());
				});

				It("NewPath cannot contain subobjects", [this]
				{
					const FSoftObjectPath OldSubobject(TEXT("/Game/Map.Map:PersistentLevel.OldActor.Subobject"));
					const FSoftObjectPath NewPath(TEXT("/Game/Map.Map:PersistentLevel.NewActor.Subobject"));
					const TOptional<FSoftObjectPath> Replacement = ConcertSyncCore::ReplaceActorInPath(OldSubobject, NewPath);
					TestFalse(TEXT("Invalid"), Replacement.IsSet());
				});
			});
		});
	}
}
