// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/SendReceiveWorldTestBase.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"

namespace UE::ConcertSyncTests::Replication::Bugs
{
	/**
	 * This tests that the replication system detects when a replicated object is renamed and subsequently replaced by an equally named object.
	 * 
	 * This happens for example when replicating a construction script created component on a Blueprint actor and the actor runs a construction script,
	 * which reinstances the components.
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FChangeObjectNameDuringReplication, FSendReceiveWorldTestBase, "VirtualProduction.Concert.Replication.Bugs.ChangeObjectNameDuringReplication", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FChangeObjectNameDuringReplication::RunTest(const FString& Parameters)
	{
		// The real replication bridge must be used because we're testing that the real bridge detects renaming
		SetUpClientAndServer(ESendReceiveTestFlags::UseRealReplicationBridge);
		SetupWorld();

		const FVector InitialLocation { 10.f };
		const FVector GarbageLocation { -10.f };
		const FVector AfterRenameLocation { 20.f };
		const FName ComponentName = TEXT("ReplicatedComponent");

		// Create an actor a component called "ReplicatedComponent"
		UWorld* World = GetWorld();
		AStaticMeshActor* StaticMeshActor = World->SpawnActor<AStaticMeshActor>();
		USceneComponent* FirstComponent = NewObject<USceneComponent>(StaticMeshActor, ComponentName);
		FirstComponent->SetupAttachment(StaticMeshActor->GetRootComponent());
		FirstComponent->RegisterComponent();
		FirstComponent->SetRelativeLocation(InitialLocation);

		// Validate properties exist (in case FConcertPropertyChain implementation is broken)
		const TOptional<FConcertPropertyChain> RelativeLocation = FConcertPropertyChain::CreateFromPath(*USceneComponent::StaticClass(), { TEXT("RelativeLocation") });
		const TOptional<FConcertPropertyChain> RelativeLocationX = FConcertPropertyChain::CreateFromPath(*USceneComponent::StaticClass(), { TEXT("RelativeLocation"), TEXT("X") });
		const TOptional<FConcertPropertyChain> RelativeLocationY = FConcertPropertyChain::CreateFromPath(*USceneComponent::StaticClass(), { TEXT("RelativeLocation"), TEXT("Y") });
		const TOptional<FConcertPropertyChain> RelativeLocationZ = FConcertPropertyChain::CreateFromPath(*USceneComponent::StaticClass(), { TEXT("RelativeLocation"), TEXT("Z") });
		if (!RelativeLocation || !RelativeLocationX || !RelativeLocationY || !RelativeLocationZ)
		{
			AddError(TEXT("Failed to find RelativeLocation properties. Check FConcertPropertyChain implementation."));
			return false;
		}
		
		// Register the component for replication
		FConcertReplication_ChangeStream_Request StreamRequest;
		FConcertReplicationStream Stream { FGuid::NewGuid() };
		Stream.BaseDescription.FrequencySettings.Defaults.ReplicationMode = EConcertObjectReplicationMode::Realtime;
		FConcertReplicatedObjectInfo& ObjectInfo = Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(FirstComponent);
		ObjectInfo.ClassPath = FirstComponent->GetClass();
		ObjectInfo.PropertySelection.ReplicatedProperties.Append({ *RelativeLocation, *RelativeLocationX, *RelativeLocationY, *RelativeLocationZ });
		StreamRequest.StreamsToAdd = { Stream };
		ClientReplicationManager_Sender->ChangeStream(StreamRequest)
			.Next([this](FConcertReplication_ChangeStream_Response&& Response)
			{
				TestTrue(TEXT("Register component"), Response.IsSuccess());
			});
		ClientReplicationManager_Sender->TakeAuthorityOver({ FirstComponent })
			.Next([this](FConcertReplication_ChangeAuthority_Response&& Response)
			{
				TestTrue(TEXT("Take authority of component"), Response.RejectedObjects.IsEmpty());
			});

		// Replicate the original component
		{
			auto SetTestValues = [&](UWorld&)
			{
				FirstComponent->SetRelativeLocation(InitialLocation);
			};
			auto SetGarbageValues = [&](UWorld&)
			{
				FirstComponent->SetRelativeLocation(GarbageLocation);
			};
			auto TestValues = [&](UWorld&)
			{
				TestEqual(TEXT("Replicate original component"), FirstComponent->GetRelativeLocation(), InitialLocation);
			};
			SimulateSenderToReceiverForWorld(SetTestValues, SetGarbageValues, { TestValues });
		}

		// Rename the component and replace it. This simulates running the construction script.
		FirstComponent->UnregisterComponent();
		StaticMeshActor->RemoveOwnedComponent(FirstComponent);

		FirstComponent->Rename(*FString::Printf(TEXT("TestGarbage_%s"), *FGuid::NewGuid().ToString()), GetTransientPackage());
		USceneComponent* NewComponent = NewObject<USceneComponent>(StaticMeshActor, ComponentName);
		NewComponent->RegisterComponent();

		// Replicate the new component with same name
		{
			auto SetTestValues = [&](UWorld&)
			{
				NewComponent->SetRelativeLocation(AfterRenameLocation);
			};
			auto SetGarbageValues = [&](UWorld&)
			{
				NewComponent->SetRelativeLocation(GarbageLocation);
			};
			auto TestValues = [&](UWorld&)
			{
				TestEqual(TEXT("Replicate new component"), NewComponent->GetRelativeLocation(), AfterRenameLocation);
			};
			SimulateSenderToReceiverForWorld(SetTestValues, SetGarbageValues, { TestValues });
		}
		
		return true;
	}
}
