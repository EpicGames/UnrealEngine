// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	BEGIN_DEFINE_SPEC(FRevertChange, "VirtualProduction.Concert.Replication.Bugs.RevertRequest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> Replicator;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client1;

		const FGuid StreamId = FGuid::NewGuid();
	END_DEFINE_SPEC(FRevertChange);

	/** There used to be a bug where the client would revert a failed authority request incorrectly ending up thinking it has authority even though it does not. */
	void FRevertChange::Define()
	{
		BeforeEach([this]
		{
			Replicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Client1 = &Server->ConnectClient();

			Client1->JoinReplication(Replicator->CreateSenderArgs(StreamId));
		});
		AfterEach([this]
		{
			Replicator.Reset();
			Server.Reset();
		});

		It("Reverting a stream change operation does not grant authority if the request fails", [this]
		{
			// Simulate timeout
			const TSharedRef<FConcertServerSessionMock>& Session = Server->GetServerSessionMock();
			Session->SetTestFlags(EServerSessionTestingFlags::AllowRequestTimeouts);
			Session->UnregisterCustomRequestHandler<FConcertReplication_ChangeStream_Request>();

			// The revert mechanism used to blindly add back authority for each removed object instead of checking whether it actually had authority when the request was made.
			IConcertClientReplicationManager& ReplicationManager = Client1->GetClientReplicationManager();
			ReplicationManager.ChangeStream({ .ObjectsToRemove = { { StreamId, Replicator->TestObject  } }});
			TestFalse(TEXT("Receive authority"), ReplicationManager.HasAuthorityOver(Replicator->TestObject));
		});
		
		It("Reverting a release authority operation does not grant authority if the request fails", [this]
		{
			// Simulate timeout
			const TSharedRef<FConcertServerSessionMock>& Session = Server->GetServerSessionMock();
			Session->SetTestFlags(EServerSessionTestingFlags::AllowRequestTimeouts);
			Session->UnregisterCustomRequestHandler<FConcertReplication_ChangeAuthority_Request>();

			// The revert mechanism used to blindly add back authority for each requested object instead of checking whether it actually had authority when the request was made.
			IConcertClientReplicationManager& ReplicationManager = Client1->GetClientReplicationManager();
			ReplicationManager.ReleaseAuthorityOf({ Replicator->TestObject });
			TestFalse(TEXT("Receive authority"), ReplicationManager.HasAuthorityOver(Replicator->TestObject));
		});
	}
}