// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Replication/Util/Spec/ClientEventCounter.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	BEGIN_DEFINE_SPEC(FChangeStreamAndAuthorityEvents, "VirtualProduction.Concert.Replication.ClientManager.ChangeStreamAndAuthorityEvents", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> Replicator;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Sender;

		const FGuid StreamId = FGuid::NewGuid();
	
		/** Will help us count the client events. */
		FClientEventCounter EventCounter;
	END_DEFINE_SPEC(FChangeStreamAndAuthorityEvents);

	/**
	 * This tests that IConcertClientReplicationManager stream and authority change events are made correctly in response to
	 * IConcertClientReplicationManager::ChangeStream.
	 */
	void FChangeStreamAndAuthorityEvents::Define()
	{
		BeforeEach([this]
		{
			Replicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Sender = &Server->ConnectClient();

			Sender->JoinReplication();
			EventCounter.Subscribe(*Sender);
		});
		AfterEach([this]
		{
			Replicator.Reset();
			Server.Reset();
		});

		It("When adding a stream, broadcast stream event", [this]
		{
			Sender->GetClientReplicationManager().ChangeStream({ .StreamsToAdd = { Replicator->CreateStream(StreamId) } });
			EventCounter.TestCount(*this, 1, 0);
		});

		It("When taking authority, broadcast authority event", [this]
		{
			Sender->GetClientReplicationManager().ChangeStream({ .StreamsToAdd = { Replicator->CreateStream(StreamId) } });
			EventCounter.ResetEventCount();
			
			Sender->GetClientReplicationManager().TakeAuthorityOver({ Replicator->TestObject });
			EventCounter.TestCount(*this, 0, 1);
		});
		
		It("When removing owned object, broadcast stream and authority event", [this]
		{
			Sender->GetClientReplicationManager().ChangeStream({ .StreamsToAdd = { Replicator->CreateStream(StreamId) } });
			Sender->GetClientReplicationManager().TakeAuthorityOver({ Replicator->TestObject });
			EventCounter.ResetEventCount();

			Sender->GetClientReplicationManager().ChangeStream({ .StreamsToRemove = { StreamId } });
			EventCounter.TestCount(*this, 1, 1);
		});
	}
}