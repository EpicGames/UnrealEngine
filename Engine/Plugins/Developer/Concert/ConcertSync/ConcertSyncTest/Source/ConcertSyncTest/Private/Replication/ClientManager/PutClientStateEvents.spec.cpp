// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Replication/Util/Spec/ClientEventCounter.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	BEGIN_DEFINE_SPEC(FPutClientStateEvents, "VirtualProduction.Concert.Replication.ClientManager.PutClientStateEvents", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> Replicator_Client1;
		TUniquePtr<FObjectTestReplicator> Replicator_Client2;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client1;
		FReplicationClient* Client2;

		const FGuid StreamId = FGuid::NewGuid();
	
		/** Will help us count the client events. */
		FClientEventCounter EventCounter_Client1;
		/** Will help us count the client events. */
		FClientEventCounter EventCounter_Client2;
	
		FConcertReplication_PutState_Request BuildBaseRequest() const
		{
			FConcertReplication_PutState_Request Request;

			FConcertReplicationStreamArray StreamContent_Client1;
			StreamContent_Client1.Streams = { Replicator_Client1->CreateStream(StreamId) };
			Request.NewStreams.Add(Client1->GetEndpointId(), StreamContent_Client1);
			
			FConcertReplicationStreamArray StreamContent_Client2;
			StreamContent_Client2.Streams = { Replicator_Client2->CreateStream(StreamId) };
			Request.NewStreams.Add(Client2->GetEndpointId(), StreamContent_Client2);

			const FConcertObjectInStreamID ObjectId_Client1 = { StreamId, Replicator_Client1->TestObject };
			const FConcertObjectInStreamID ObjectId_Client2 = { StreamId, Replicator_Client2->TestObject };
			Request.NewAuthorityState.Add(Client1->GetEndpointId(), { { ObjectId_Client1 } });
			Request.NewAuthorityState.Add(Client2->GetEndpointId(), { { ObjectId_Client2 } });
				
			return Request;
		}
	END_DEFINE_SPEC(FPutClientStateEvents);

	/**
	 * This tests that IConcertClientReplicationManager stream and authority change events are made correctly in response to
	 * IConcertClientReplicationManager::PutClientState.
	 */
	void FPutClientStateEvents::Define()
	{
		BeforeEach([this]
		{
			Replicator_Client1 = MakeUnique<FObjectTestReplicator>(TEXT("Foo"));
			Replicator_Client2 = MakeUnique<FObjectTestReplicator>(TEXT("Bar"));
			Server = MakeUnique<FReplicationServer>(*this);
			Client1 = &Server->ConnectClient();
			Client2 = &Server->ConnectClient();

			Client1->JoinReplication();
			Client2->JoinReplication();
			EventCounter_Client1.Subscribe(*Client1);
			EventCounter_Client2.Subscribe(*Client2);
		});
		AfterEach([this]
		{
			Replicator_Client1.Reset();
			Server.Reset();
		});

		It("When PutState is applied on an empty client, broadcast stream and authority event once.", [this]
		{
			Client1->GetClientReplicationManager().PutClientState(BuildBaseRequest());
			EventCounter_Client1.TestCount(*this, 1, 1);
			EventCounter_Client2.TestCount(*this, 1, 1);
		});
	}
}