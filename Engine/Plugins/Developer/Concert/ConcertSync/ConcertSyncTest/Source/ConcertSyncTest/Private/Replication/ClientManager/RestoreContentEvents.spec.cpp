// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Replication/Util/Spec/ClientEventCounter.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	static FConcertReplication_RestoreContent_Response CreateRestoreContentFrom(
		const FObjectTestReplicator& Replication,
		bool bGiveAuthority = true,
		const FGuid& StreamId = FGuid::NewGuid()
		)
	{
		FConcertReplication_RestoreContent_Response Result;

		Result.ErrorCode = EConcertReplicationRestoreErrorCode::Success;
		Result.ClientInfo.Streams = { Replication.CreateStream(StreamId).BaseDescription };
		if (bGiveAuthority)
		{
			Result.ClientInfo.Authority = { FConcertAuthorityClientInfo{ StreamId, { Replication.TestObject } } };
		}
		
		return Result;
	}
	
	BEGIN_DEFINE_SPEC(FRestoreContentEvents, "VirtualProduction.Concert.Replication.ClientManager.RestoreContentEvents", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> Replicator;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Sender;

		/** Will help us count the client events. */
		FClientEventCounter EventCounter;

		FConcertReplication_RestoreContent_Response ResponseToGive;
		EConcertSessionResponseCode HandleRestoreContentRequest(
			const FConcertSessionContext& Context,
			const FConcertReplication_RestoreContent_Request& Request,
			FConcertReplication_RestoreContent_Response& Response
			)
		{
			Response = ResponseToGive;
			return EConcertSessionResponseCode::Success;
		}
	END_DEFINE_SPEC(FRestoreContentEvents);

	/**
	 * This tests that IConcertClientReplicationManager stream and authority change events are made correctly in response to
	 * IConcertClientReplicationManager::RestoreContent.
	 */
	void FRestoreContentEvents::Define()
	{
		BeforeEach([this]
		{
			Replicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Sender = &Server->ConnectClient();

			Server->GetServerSessionMock()->RegisterCustomRequestHandler<FConcertReplication_RestoreContent_Request, FConcertReplication_RestoreContent_Response>(
				this, &FRestoreContentEvents::HandleRestoreContentRequest
				);
			ResponseToGive = { EConcertReplicationRestoreErrorCode::Success };

			Sender->JoinReplication();
			EventCounter.Subscribe(*Sender);
		});
		AfterEach([this]
		{
			Replicator.Reset();
			Server.Reset();
		});

		It("When content is restored, events called", [this]
		{
			ResponseToGive = CreateRestoreContentFrom(*Replicator);
			Sender->GetClientReplicationManager().RestoreContent();
			EventCounter.TestCount(*this, 1, 1);
		});
		It("When only stream changes, only stream change event is called", [this]
		{
			constexpr bool bGiveAuthority = false;
			ResponseToGive = CreateRestoreContentFrom(*Replicator, bGiveAuthority);
			Sender->GetClientReplicationManager().RestoreContent();
			EventCounter.TestCount(*this, 1, 0);
		});
		It("When only authority changes, only authority change event is called", [this]
		{
			const FGuid& StreamId = FGuid::NewGuid();
			
			ResponseToGive = CreateRestoreContentFrom(*Replicator, true, StreamId);
			Sender->GetClientReplicationManager().RestoreContent();
			EventCounter.ResetEventCount();
			
			ResponseToGive = CreateRestoreContentFrom(*Replicator, false, StreamId);
			Sender->GetClientReplicationManager().RestoreContent();
			EventCounter.TestCount(*this, 0, 1);
		});

		It("When empty content is restored, no events called", [this]
		{
			ResponseToGive = { EConcertReplicationRestoreErrorCode::Success };
			Sender->GetClientReplicationManager().RestoreContent();
			EventCounter.TestCount(*this, 0, 0);
		});
		It("When same content is restored again, no events called", [this]
		{
			ResponseToGive = CreateRestoreContentFrom(*Replicator);
			Sender->GetClientReplicationManager().RestoreContent();
			EventCounter.ResetEventCount();
			
			// This should generate no events because it's the same content.
			Sender->GetClientReplicationManager().RestoreContent();
			EventCounter.TestCount(*this, 0, 0);
		});

		It("When RestoreContent times out, no events called ", [this]
		{
			ResponseToGive = { EConcertReplicationRestoreErrorCode::Timeout };
			Sender->GetClientReplicationManager().RestoreContent();
			EventCounter.TestCount(*this, 0, 0);
		});
	}
}