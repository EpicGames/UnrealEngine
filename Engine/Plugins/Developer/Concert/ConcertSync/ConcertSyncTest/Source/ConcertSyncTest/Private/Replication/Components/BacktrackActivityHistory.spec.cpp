// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncSessionTypes.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Misc/ClientPredictionUtils.h"
#include "Util/Mock/ConcertClientWorkspaceBaseMock.h"

#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::BacktrackActivityHistory
{
	struct FTestCase
	{
		const FGuid TargetClientId { 1, 0, 0, 0 };
		const FGuid OtherClientId { 2, 0, 0, 0 };
		const FGuid TargetStreamId { 0, 1, 0, 0 };
		const FGuid OtherStreamId { 0, 2, 0, 0 };
	
		const TUniquePtr<FObjectTestReplicator> TargetReplicator = MakeUnique<FObjectTestReplicator>(TEXT("TargetObject"));
		const TUniquePtr<FObjectTestReplicator> OtherReplicator = MakeUnique<FObjectTestReplicator>(TEXT("OtherObject"));

		using EEventType = EConcertSyncActivityEventType;
		using FActivity = FConcertSyncActivity;
		const TArray<FConcertSyncActivity> Activities
		{
			FActivity{ .ActivityId = 1,						.EndpointId = OtherClientId,  .EventType = EEventType::Replication,	.EventId = 1 },
			FActivity{ .ActivityId = 2,						.EndpointId = TargetClientId, .EventType = EEventType::Replication,	.EventId = 2 },
			FActivity{ .ActivityId = 3, .bIgnored = true,	.EndpointId = TargetClientId, .EventType = EEventType::Replication,	.EventId = 3 },
			FActivity{ .ActivityId = 4,						.EndpointId = TargetClientId, .EventType = EEventType::Replication,	.EventId = 4 },
			FActivity{ .ActivityId = 5,						.EndpointId = OtherClientId,  .EventType = EEventType::Replication,	.EventId = 5 },
			FActivity{ .ActivityId = 6,						.EndpointId = TargetClientId, .EventType = EEventType::Connection,	.EventId = 1 },
			FActivity{ .ActivityId = 7,						.EndpointId = TargetClientId, .EventType = EEventType::Package,		.EventId = 1 },
			FActivity{ .ActivityId = 8,						.EndpointId = TargetClientId, .EventType = EEventType::Transaction,	.EventId = 1 },
			FActivity{ .ActivityId = 9,						.EndpointId = TargetClientId, .EventType = EEventType::Lock,		.EventId = 1 },
		};
	
		const FConcertReplicationStream TargetStream = TargetReplicator->CreateStream(TargetStreamId);
		const FConcertReplicationStream OtherStream = TargetReplicator->CreateStream(OtherStreamId);
		const TArray<FConcertObjectInStreamID> TargetAuthority { { TargetStreamId, TargetReplicator->TestObject } };
		const TArray<FConcertObjectInStreamID> OtherAuthority { { OtherStreamId, OtherReplicator->TestObject } };
		const FConcertSyncReplicationPayload_LeaveReplication TargetPayload{ .Streams = { TargetStream }, .OwnedObjects = TargetAuthority };
		const FConcertSyncReplicationPayload_LeaveReplication OtherPayload{ .Streams = { OtherStream }, .OwnedObjects = OtherAuthority };
		
		TOptional<FConcertSyncReplicationEvent> GetReplicationEvent(const int64 EventId, FAutomationTestBase& Test) const
		{
			switch (EventId)
			{
			case 1: Test.AddError(TEXT("Iterated from the front instead of from back")); return {};
			case 2: return FConcertSyncReplicationEvent(TargetPayload);
			case 3: return FConcertSyncReplicationEvent(OtherPayload);
			case 4: return FConcertSyncReplicationEvent(FConcertSyncReplicationPayload_Mute{});
			case 5: return FConcertSyncReplicationEvent(OtherPayload);
			default: Test.AddError(TEXT("Unexpected EventId")); return {};
			}
		}

		void TestContentAsExpected(
			FAutomationTestBase& Test,
			const TOptional<int64>& ActivityId,
			const TArray<FConcertBaseStreamInfo>& Streams,
			const TArray<FConcertObjectInStreamID>& Authority
			) const
		{
			if (Streams.Num() != 1 || Authority.Num() != 1 || !ActivityId.IsSet())
			{
				Test.AddError(TEXT("Unexpected number"));
			}
			else
			{
				Test.TestEqual(TEXT("ActivityId"), *ActivityId, static_cast<int64>(2));
				Test.TestEqual(TEXT("Streams"), Streams[0], TargetStream.BaseDescription);
				Test.TestEqual(TEXT("Authority"), Authority, TargetAuthority);
			}
		}
	};
	
	class FBacktrackClientWorkspaceMock : public FConcertClientWorkspaceBaseMock
	{
		const FTestCase& TestData;
		const TMap<FGuid, FConcertClientInfo> Endpoints;
		FAutomationTestBase& TestInstance;
	public:
		
		FBacktrackClientWorkspaceMock(const FTestCase& TestData, TMap<FGuid, FConcertClientInfo> Endpoints, FAutomationTestBase& TestInstance)
			: TestData(TestData)
			, Endpoints(MoveTemp(Endpoints))
			, TestInstance(TestInstance)
		{}

		//~ Begin IConcertClientWorkspace Interface
		virtual int64 GetLastActivityId() const override { return 9; }
		virtual void GetActivities(const int64 FirstActivityIdToFetch, const int64 MaxNumActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, TArray<FConcertSessionActivity>& OutActivities) const override
		{
			// The IDs are 1 larger than their array indices
			const int32 IdToIndex = FirstActivityIdToFetch - 1;
			if (TestData.Activities.IsValidIndex(IdToIndex))
			{
				check(OutActivities.IsEmpty());
				for (int32 i = IdToIndex; OutActivities.Num() < MaxNumActivities && TestData.Activities.IsValidIndex(i); ++i)
				{
					OutActivities.Emplace(TestData.Activities[i], FStructOnScope{});
				}
				OutEndpointClientInfoMap = Endpoints;
			}
		}
		virtual bool FindReplicationEvent(const int64 ReplicationEventId, FConcertSyncReplicationEvent& OutReplicationEvent) const override
		{
			if (const TOptional<FConcertSyncReplicationEvent> Event = TestData.GetReplicationEvent(ReplicationEventId, TestInstance))
			{
				OutReplicationEvent = *Event;
				return true;
			}
			return false;
		}
		//~ End IConcertClientWorkspace Interface
	};
	
	BEGIN_DEFINE_SPEC(FBacktrackActivityHistory, "VirtualProduction.Concert.Replication.Components.BacktrackActivityHistory", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FTestCase> TestData;
	END_DEFINE_SPEC(FBacktrackActivityHistory);

	void FBacktrackActivityHistory::Define()
	{
		BeforeEach([this]
		{
			TestData = MakeUnique<FTestCase>();
		});
		AfterEach([this]
		{
			TestData.Reset();
		});
		
		It("BacktrackActivityHistoryForActivityThatSetsContent", [this]
		{
			TArray<FConcertBaseStreamInfo> Streams;
			TArray<FConcertObjectInStreamID> Authority;
			const TOptional<int64> ActivityId = ConcertSyncCore::Replication::BacktrackActivityHistoryForActivityThatSetsContent(
				TestData->Activities,
				[this](const FGuid& EndpointId){ return EndpointId == TestData->TargetClientId; },
				[this](const int64 EventId, TFunctionRef<void(const FConcertSyncReplicationEvent& Event)> Callback)
				{
					if (const TOptional<FConcertSyncReplicationEvent> Event = TestData->GetReplicationEvent(EventId, *this))
					{
						Callback(*Event);
					}
				},
				Streams, Authority
				);

			TestData->TestContentAsExpected(*this, ActivityId, Streams, Authority);
		});

		Describe("IncrementalBacktrackActivityHistoryForActivityThatSetsContent", [this]
		{
			const auto TestHasExpectedContent = [this](const int64 MaxToFetch = 50)
			{
				const FConcertClientInfo TargetClientInfo { .DeviceName = TEXT("TargetDevice"), .DisplayName = TEXT("TargetName") };
				const FConcertClientInfo OtherClientInfo { .DeviceName = TEXT("OtherDevice"), .DisplayName = TEXT("OtherName") };
				const TMap<FGuid, FConcertClientInfo> Endpoints
				{
					{ TestData->TargetClientId, TargetClientInfo },
					{ TestData->OtherClientId, OtherClientInfo }
				};
				FBacktrackClientWorkspaceMock WorkspaceMock(*TestData, Endpoints, *this);
				
				TArray<FConcertBaseStreamInfo> Streams;
				TArray<FConcertObjectInStreamID> Authority;
				const TOptional<int64> ActivityId = ConcertSyncClient::Replication::IncrementalBacktrackActivityHistoryForActivityThatSetsContent(
					WorkspaceMock, TargetClientInfo, Streams, Authority, MaxToFetch
					);

				TestData->TestContentAsExpected(*this, ActivityId, Streams, Authority);
			};
			It("MaxToFetch = 0", [this, TestHasExpectedContent]{ TestHasExpectedContent(0); });
			It("MaxToFetch = 1", [this, TestHasExpectedContent]{ TestHasExpectedContent(1); });
			It("MaxToFetch = 2", [this, TestHasExpectedContent]{ TestHasExpectedContent(2); });
			It("MaxToFetch = 3", [this, TestHasExpectedContent]{ TestHasExpectedContent(3); });
			It("MaxToFetch = 4", [this, TestHasExpectedContent]{ TestHasExpectedContent(4); });
			It("MaxToFetch = 5", [this, TestHasExpectedContent]{ TestHasExpectedContent(5); });
			It("MaxToFetch = 6", [this, TestHasExpectedContent]{ TestHasExpectedContent(6); });
			It("MaxToFetch = 7", [this, TestHasExpectedContent]{ TestHasExpectedContent(7); });
			It("MaxToFetch = 8", [this, TestHasExpectedContent]{ TestHasExpectedContent(8); });
			It("MaxToFetch = 9", [this, TestHasExpectedContent]{ TestHasExpectedContent(9); });
			It("MaxToFetch = 10", [this, TestHasExpectedContent]{ TestHasExpectedContent(10); });
			It("MaxToFetch = default", [this, TestHasExpectedContent]{ TestHasExpectedContent(); });
			
			const auto TestDoesNotHaveContent = [this](const int64 ActivityIdCutoff = 1, const int64 MaxToFetch = 50)
			{
				const FConcertClientInfo TargetClientInfo { .DeviceName = TEXT("TargetDevice"), .DisplayName = TEXT("TargetName") };
				const FConcertClientInfo OtherClientInfo { .DeviceName = TEXT("OtherDevice"), .DisplayName = TEXT("OtherName") };
				const TMap<FGuid, FConcertClientInfo> Endpoints
				{
					{ TestData->TargetClientId, TargetClientInfo },
					{ TestData->OtherClientId, OtherClientInfo }
				};
				FBacktrackClientWorkspaceMock WorkspaceMock(*TestData, Endpoints, *this);
				
				TArray<FConcertBaseStreamInfo> Streams;
				TArray<FConcertObjectInStreamID> Authority;
				const TOptional<int64> ActivityId = ConcertSyncClient::Replication::IncrementalBacktrackActivityHistoryForActivityThatSetsContent(
					WorkspaceMock, TargetClientInfo, Streams, Authority, MaxToFetch, ActivityIdCutoff
					);

				TestFalse(TEXT("ActivityId.IsSet()"), ActivityId.IsSet());
				TestEqual(TEXT("Streams.Num()"), Streams.Num(), 0);
				TestEqual(TEXT("Authority.Num()"), Authority.Num(), 0);
			};
			It("ActivityIdCutoff = 3", [this, TestDoesNotHaveContent]{ TestDoesNotHaveContent(3); });
			It("ActivityIdCutoff = 4", [this, TestDoesNotHaveContent]{ TestDoesNotHaveContent(4); });
			It("ActivityIdCutoff = 5", [this, TestDoesNotHaveContent]{ TestDoesNotHaveContent(5); });
			It("ActivityIdCutoff = 6", [this, TestDoesNotHaveContent]{ TestDoesNotHaveContent(6); });
			It("ActivityIdCutoff = 7", [this, TestDoesNotHaveContent]{ TestDoesNotHaveContent(7); });
			It("ActivityIdCutoff = 8", [this, TestDoesNotHaveContent]{ TestDoesNotHaveContent(8); });
			It("ActivityIdCutoff = 9", [this, TestDoesNotHaveContent]{ TestDoesNotHaveContent(9); });
			It("ActivityIdCutoff = 10", [this, TestDoesNotHaveContent]{ TestDoesNotHaveContent(10); });
		});
	}
}
