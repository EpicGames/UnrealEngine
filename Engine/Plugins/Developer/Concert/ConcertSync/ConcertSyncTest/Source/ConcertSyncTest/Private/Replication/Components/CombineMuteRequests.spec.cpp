// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/ObjectPathOuterIterator.h"
#include "Replication/MuteUtils.h"

#include "Replication/Util/Spec/ObjectTestReplicator.h"

namespace UE::ConcertSyncTests::Replication
{
	class FTestMuteStateGroundTruth : public ConcertSyncCore::Replication::MuteUtils::IMuteStateGroundTruth
	{
	public:

		TMap<FSoftObjectPath, ConcertSyncCore::Replication::MuteUtils::EMuteState> MuteStates;
		TMap<FSoftObjectPath, FConcertReplication_ObjectMuteSetting> ExplicitStates;

		void AddObject(
			const FSoftObjectPath& ObjectPath,
			ConcertSyncCore::Replication::MuteUtils::EMuteState MuteState = ConcertSyncCore::Replication::MuteUtils::EMuteState::None
			)
		{
			MuteStates.Add(ObjectPath, MuteState);
		}

		void AddExplicitObject(const FSoftObjectPath& ObjectPath, ConcertSyncCore::Replication::MuteUtils::EMuteState MuteState, const FConcertReplication_ObjectMuteSetting& Setting)
		{
			check(MuteState == ConcertSyncCore::Replication::MuteUtils::EMuteState::ExplicitlyMuted || MuteState == ConcertSyncCore::Replication::MuteUtils::EMuteState::ExplicitlyUnmuted);
			AddObject(ObjectPath, MuteState);
			ExplicitStates.Add(ObjectPath, Setting);
		}
		
		virtual ConcertSyncCore::Replication::MuteUtils::EMuteState GetMuteState(const FSoftObjectPath& Object) const override
		{
			const ConcertSyncCore::Replication::MuteUtils::EMuteState* MuteState = MuteStates.Find(Object);
			return MuteState ? *MuteState : ConcertSyncCore::Replication::MuteUtils::EMuteState::None;
		}

		virtual TOptional<FConcertReplication_ObjectMuteSetting> GetExplicitSetting(const FSoftObjectPath& Object) const override
		{
			const FConcertReplication_ObjectMuteSetting* Setting = ExplicitStates.Find(Object);
			return Setting ? *Setting : TOptional<FConcertReplication_ObjectMuteSetting>{};
		}

		virtual bool IsObjectKnown(const FSoftObjectPath& Object) const override
		{
			if (MuteStates.Contains(Object))
			{
				return true;
			}
		
			for (ConcertSyncCore::FObjectPathOuterIterator It(Object); It; ++It)
			{
				if (MuteStates.Contains(*It))
				{
					return true;
				}
			}
			
			return false;
		}
	};
	
	/** This tests that muting & unmuting retains a consistent check across various types of changes. */
	BEGIN_DEFINE_SPEC(FCombineMuteRequestsSpec, "VirtualProduction.Concert.Replication.Components.CombineMuteRequests", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		FGuid StreamId					= FGuid::NewGuid();
		// Leverage FObjectTestReplicator to create more UObjects
		TSharedPtr<FObjectTestReplicator> Replicator_Foo;			
		TSharedPtr<FObjectTestReplicator> Replicator_FooSubobject;		
		TSharedPtr<FObjectTestReplicator> Replicator_FooNestedSubobject;
	
		TSharedPtr<FObjectTestReplicator> Replicator_Bar;			

		TUniquePtr<FTestMuteStateGroundTruth> GroundTruth;
	END_DEFINE_SPEC(FCombineMuteRequestsSpec);

	/** This tests that muting requests work when EConcertSyncSessionFlags::ShouldAllowGlobalMuting is set. */
	void FCombineMuteRequestsSpec::Define()
	{
		using namespace ConcertSyncCore::Replication::MuteUtils;
		
		BeforeEach([this]
		{
			Replicator_Foo					= MakeShared<FObjectTestReplicator>();
			Replicator_FooSubobject			= Replicator_Foo->CreateSubobjectReplicator();
			Replicator_FooNestedSubobject	= Replicator_FooSubobject->CreateSubobjectReplicator();
			Replicator_Bar					= MakeShared<FObjectTestReplicator>();
			
			GroundTruth						= MakeUnique<FTestMuteStateGroundTruth>();
			GroundTruth->AddObject(Replicator_Foo->TestObject);
		});
		AfterEach([this]
		{
			// Test would hold onto this for rest of engine lifetime. Clean up this mini would-be leak.
			Replicator_Foo.Reset();
			Replicator_FooSubobject.Reset();
			Replicator_FooNestedSubobject.Reset();
			Replicator_Bar.Reset();
			GroundTruth.Reset();
		});

		It("Combine {} with 'Mute Foo & Subobject'", [this]
		{
			FConcertReplication_ChangeMuteState_Request BaseRequest;
			FConcertReplication_ChangeMuteState_Request MuteFoo {
				.ObjectsToMute = {
					{ Replicator_Foo->TestObject, { EConcertReplicationMuteOption::OnlyObject } },
					{ Replicator_FooSubobject->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects } }
				} };
			CombineMuteRequests(BaseRequest, MuteFoo, *GroundTruth);

			TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 2);
			TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 0);
			const FConcertReplication_ObjectMuteSetting* MuteSetting_Foo = BaseRequest.ObjectsToMute.Find(Replicator_Foo->TestObject);
			const FConcertReplication_ObjectMuteSetting* MuteSetting_Subobject = BaseRequest.ObjectsToMute.Find(Replicator_FooSubobject->TestObject);
			TestTrue(TEXT("MuteSetting_Foo"), MuteSetting_Foo && MuteSetting_Foo->Flags == EConcertReplicationMuteOption::OnlyObject);
			TestTrue(TEXT("MuteSetting_Foo"), MuteSetting_Subobject && MuteSetting_Subobject->Flags == EConcertReplicationMuteOption::ObjectAndSubobjects);
		});

		Describe("Combine 'Mute Foo' with 'Unmute Foo' yields empty request", [this]
		{
			const auto RunTest = [this](const int32 NumExpectedUnmutedObjects)
			{
				FConcertReplication_ChangeMuteState_Request BaseRequest { .ObjectsToMute = {{ Replicator_Foo->TestObject, { EConcertReplicationMuteOption::OnlyObject } } } };
				FConcertReplication_ChangeMuteState_Request UnmuteFoo { .ObjectsToUnmute = { { Replicator_Foo->TestObject, { EConcertReplicationMuteOption::OnlyObject } } } };
				CombineMuteRequests(BaseRequest, UnmuteFoo, *GroundTruth);
				
				TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 0);
				TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), NumExpectedUnmutedObjects);
			};
			
			It("EMuteState::None", [this, RunTest]
			{
				GroundTruth->AddObject(Replicator_Foo->TestObject, EMuteState::None);
				RunTest(0);
			});
			It("EMuteState::ExplicitlyMuted (ObjectAndSubobjects)", [this, RunTest]
			{
				GroundTruth->AddExplicitObject(Replicator_Foo->TestObject, EMuteState::ExplicitlyMuted, { EConcertReplicationMuteOption::ObjectAndSubobjects });
				// RunTest unmutes with the OnlyObject option, whic is different than the ground truth, so expect a change
				RunTest(1);
			});
			It("EMuteState::ExplicitlyMuted (OnlyObject)", [this, RunTest]
			{
				GroundTruth->AddExplicitObject(Replicator_Foo->TestObject, EMuteState::ExplicitlyMuted, { EConcertReplicationMuteOption::OnlyObject });
				// RunTest unmutes with the OnlyObject option, whic is equal to the ground truth, so expect no change
				RunTest(0);
			});
		});

		It("Combine 'Unmute Foo' with 'Mute Foo' yields empty request", [this]
		{
			GroundTruth->AddExplicitObject(Replicator_Foo->TestObject, EMuteState::ExplicitlyMuted, { EConcertReplicationMuteOption::ObjectAndSubobjects });
				
			FConcertReplication_ChangeMuteState_Request BaseRequest { .ObjectsToUnmute = {{ Replicator_Foo->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects } } } };
			FConcertReplication_ChangeMuteState_Request MuteFoo { .ObjectsToMute = { { Replicator_Foo->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects } } } };
			CombineMuteRequests(BaseRequest, MuteFoo, *GroundTruth);
		
			TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 0);
			TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 0);
		});
		
		It("Combine 'Unmute Foo' when already unmuted yields empty request", [this]
		{
			GroundTruth->AddObject(Replicator_Foo->TestObject, EMuteState::None);
				
			FConcertReplication_ChangeMuteState_Request BaseRequest;
			FConcertReplication_ChangeMuteState_Request UnmuteFoo { .ObjectsToUnmute = { { Replicator_Foo->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects } } } };
			CombineMuteRequests(BaseRequest, UnmuteFoo, *GroundTruth);
			
			TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 0);
			TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 0);
		});

		It("Combine 'Mute Foo (OnlyObject)' with 'Mute Foo (ObjectAndSubobjects)'", [this]
		{
			FConcertReplication_ChangeMuteState_Request BaseRequest { .ObjectsToMute = {{ Replicator_Foo->TestObject, { EConcertReplicationMuteOption::OnlyObject } } } };
			FConcertReplication_ChangeMuteState_Request MuteFoo { .ObjectsToMute = {{ Replicator_Foo->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects } } } };
			CombineMuteRequests(BaseRequest, MuteFoo, *GroundTruth);

			TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 1);
			TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 0);
			const FConcertReplication_ObjectMuteSetting* MuteSetting_Foo = BaseRequest.ObjectsToMute.Find(Replicator_Foo->TestObject);
			TestTrue(TEXT("MuteSetting_Foo"), MuteSetting_Foo && MuteSetting_Foo->Flags == EConcertReplicationMuteOption::ObjectAndSubobjects);
		});

		Describe("When Foo and subobjects are muted", [this]
		{
			BeforeEach([this]
			{
				GroundTruth->AddExplicitObject(Replicator_Foo->TestObject, EMuteState::ExplicitlyMuted, FConcertReplication_ObjectMuteSetting{ EConcertReplicationMuteOption::ObjectAndSubobjects });
				GroundTruth->AddObject(Replicator_FooSubobject->TestObject, EMuteState::ImplicitlyMuted);
				GroundTruth->AddObject(Replicator_FooNestedSubobject->TestObject, EMuteState::ImplicitlyMuted);
			});

			It("Cannot mute Foo and subobjects again", [this]
			{
				FConcertReplication_ChangeMuteState_Request BaseRequest;;
				FConcertReplication_ChangeMuteState_Request MuteFoo { .ObjectsToMute = {{ Replicator_Foo->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects } } } };
				CombineMuteRequests(BaseRequest, MuteFoo, *GroundTruth);
				
				TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 0);
				TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 0);
			});
			
			It("Can mute Foo with OnlyObject", [this]
			{
				FConcertReplication_ChangeMuteState_Request BaseRequest;;
				FConcertReplication_ChangeMuteState_Request UnmuteFoo { .ObjectsToMute = {{ Replicator_Foo->TestObject, { EConcertReplicationMuteOption::OnlyObject } } } };
				CombineMuteRequests(BaseRequest, UnmuteFoo, *GroundTruth);
				
				TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 1);
				TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 0);
				const FConcertReplication_ObjectMuteSetting* MuteSetting_Foo = BaseRequest.ObjectsToMute.Find(Replicator_Foo->TestObject);
				TestTrue(TEXT("MuteSetting_Foo"), MuteSetting_Foo && MuteSetting_Foo->Flags == EConcertReplicationMuteOption::OnlyObject);
			});

			It("Can unmute Subobject", [this]
			{
				FConcertReplication_ChangeMuteState_Request BaseRequest;;
				FConcertReplication_ChangeMuteState_Request MuteFoo { .ObjectsToUnmute = {{ Replicator_FooSubobject->TestObject, { EConcertReplicationMuteOption::OnlyObject } } } };
				CombineMuteRequests(BaseRequest, MuteFoo, *GroundTruth);
				
				TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 0);
				TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 1);
				const FConcertReplication_ObjectMuteSetting* MuteSetting_Subobject = BaseRequest.ObjectsToUnmute.Find(Replicator_FooSubobject->TestObject);
				TestTrue(TEXT("MuteSetting_Subobject"), MuteSetting_Subobject && MuteSetting_Subobject->Flags == EConcertReplicationMuteOption::OnlyObject);
			});
		});

		Describe("When Subobject is implicitly unmuted with ObjectAndSubobjects", [this]
		{
			BeforeEach([this]
			{
				GroundTruth->AddExplicitObject(Replicator_Foo->TestObject, EMuteState::ExplicitlyMuted, FConcertReplication_ObjectMuteSetting{ EConcertReplicationMuteOption::ObjectAndSubobjects });
				GroundTruth->AddExplicitObject(Replicator_FooSubobject->TestObject, EMuteState::ExplicitlyUnmuted, FConcertReplication_ObjectMuteSetting{ EConcertReplicationMuteOption::ObjectAndSubobjects });
				GroundTruth->AddObject(Replicator_FooNestedSubobject->TestObject, EMuteState::ImplicitlyUnmuted);
			});

			It("Cannot unmute subobject with ObjectAndSubobjects", [this]
			{
				FConcertReplication_ChangeMuteState_Request BaseRequest;;
				FConcertReplication_ChangeMuteState_Request MuteFoo { .ObjectsToUnmute = {{ Replicator_FooSubobject->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects } } } };
				CombineMuteRequests(BaseRequest, MuteFoo, *GroundTruth);
				
				TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 0);
				TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 0);
			});

			It("Can unmute subobject with OnlyObject", [this]
			{
				FConcertReplication_ChangeMuteState_Request BaseRequest;;
				FConcertReplication_ChangeMuteState_Request MuteFoo { .ObjectsToUnmute = {{ Replicator_FooSubobject->TestObject, { EConcertReplicationMuteOption::OnlyObject } } } };
				CombineMuteRequests(BaseRequest, MuteFoo, *GroundTruth);
				
				TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 0);
				TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 1);
				const FConcertReplication_ObjectMuteSetting* MuteSetting_FooSubobject = BaseRequest.ObjectsToUnmute.Find(Replicator_FooSubobject->TestObject);
				TestTrue(TEXT("MuteSetting_FooSubobject"), MuteSetting_FooSubobject && MuteSetting_FooSubobject->Flags == EConcertReplicationMuteOption::OnlyObject);
			});
			
			It("Can unmute nested subobject", [this]
			{
				FConcertReplication_ChangeMuteState_Request BaseRequest;;
				FConcertReplication_ChangeMuteState_Request MuteFoo { .ObjectsToUnmute = {{ Replicator_FooSubobject->TestObject, { EConcertReplicationMuteOption::OnlyObject } } } };
				CombineMuteRequests(BaseRequest, MuteFoo, *GroundTruth);
				
				TestEqual(TEXT("ObjectsToMute.Num()"), BaseRequest.ObjectsToMute.Num(), 0);
				TestEqual(TEXT("ObjectsToUnmute.Num()"), BaseRequest.ObjectsToUnmute.Num(), 1);
				const FConcertReplication_ObjectMuteSetting* MuteSetting_FooSubobject = BaseRequest.ObjectsToUnmute.Find(Replicator_FooSubobject->TestObject);
				TestTrue(TEXT("MuteSetting_FooSubobject"), MuteSetting_FooSubobject && MuteSetting_FooSubobject->Flags == EConcertReplicationMuteOption::OnlyObject);
			});
		});

		It("Combine skips unknown object", [this]
		{
			FConcertReplication_ChangeMuteState_Request BaseRequest1;
			FConcertReplication_ChangeMuteState_Request MuteBar { .ObjectsToMute = { { Replicator_Bar->TestObject, {} } } };
			CombineMuteRequests(BaseRequest1, MuteBar, *GroundTruth);
			TestTrue(TEXT("IsEmpty"), BaseRequest1.IsEmpty());
			
			FConcertReplication_ChangeMuteState_Request BaseRequest2;
			FConcertReplication_ChangeMuteState_Request UnmuteBar { .ObjectsToUnmute = { { Replicator_Bar->TestObject, {} } } };
			CombineMuteRequests(BaseRequest2, MuteBar, *GroundTruth);
			TestTrue(TEXT("IsEmpty"), BaseRequest2.IsEmpty());
		});
	}
}