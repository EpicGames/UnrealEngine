// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Data/ReplicationFrequencySettings.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Settings/ConcertStreamObjectAutoBindingRules.h"

#include "Components/SceneComponent.h"
#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests
{
	/** Tests that FConcertObjectReplicationSettings's < and <= work as expected. */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrequencySettingsLessEqual, "VirtualProduction.Concert.Replication.Components.FrequencySettingsLessEqual", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FFrequencySettingsLessEqual::RunTest(const FString& Parameters)
	{
		constexpr FConcertObjectReplicationSettings FPS_30 { EConcertObjectReplicationMode::SpecifiedRate, 30 }; 
		constexpr FConcertObjectReplicationSettings FPS_60 { EConcertObjectReplicationMode::SpecifiedRate, 60 };
		// Note for realtime ReplicationRate is ignored - we're only putting  30 and 60 to test that they are also ignored for comparisons
		constexpr FConcertObjectReplicationSettings Realtime_20 { EConcertObjectReplicationMode::Realtime, 20 };
		constexpr FConcertObjectReplicationSettings Realtime_60 { EConcertObjectReplicationMode::Realtime, 60 };

		TestTrue(TEXT("FPS_30 < FPS_60 == true"), FPS_30 < FPS_60);
		TestTrue(TEXT("FPS_30 <= FPS_60 == true"), FPS_30 <= FPS_60);
		TestFalse(TEXT("FPS_60 < FPS_30 == false"), FPS_60 < FPS_30);
		TestFalse(TEXT("FPS_60 <= FPS_30 == false"), FPS_60 <= FPS_30);
		
		TestTrue(TEXT("FPS_30 < FPS_60 == true"), FPS_30 < Realtime_20);
		TestTrue(TEXT("FPS_30 <= FPS_60 == true"), FPS_30 <= Realtime_20);
		TestFalse(TEXT("Realtime_20 < FPS_30 == false"), Realtime_20 < FPS_30);
		TestFalse(TEXT("Realtime_20 <= FPS_30 == false"), Realtime_20 <= FPS_30);
		
		TestTrue(TEXT("Realtime_20 <= Realtime_60 == true"), Realtime_20 <= Realtime_60);
		TestTrue(TEXT("Realtime_60 <= Realtime_20 == true"), Realtime_60 <= Realtime_20);
		TestFalse(TEXT("Realtime_20 < Realtime_60 == false"), Realtime_20 < Realtime_60);
		TestFalse(TEXT("Realtime_60 < Realtime_20 == false"), Realtime_60 < Realtime_20);
		
		return true;
	}

	/** Tests that RelativeLocation and RelativeRotation properties are discovered automatically by FConcertStreamObjectAutoBindingRules. */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoBindingPropertySettingsTest, "VirtualProduction.Concert.Replication.Components.AutoBindingPropertySettingsTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FAutoBindingPropertySettingsTest::RunTest(const FString& Parameters)
	{
		// 1. Set up: rules
		FConcertStreamObjectAutoBindingRules Rules;
		FConcertDefaultPropertySelection& PropertySelection = Rules.DefaultPropertySelection.Add(USceneComponent::StaticClass());
		PropertySelection.DefaultSelectedProperties = {
			TEXT("RelativeLocation"),
			TEXT("RelativeLocation.X"),
			TEXT("RelativeLocation.Y"),
			TEXT("RelativeLocation.Z"),
			TEXT("RelativeRotation"),
			TEXT("RelativeRotation.Pitch"),
			TEXT("RelativeRotation.Yaw"),
			TEXT("RelativeRotation.Roll")
		};

		// 2. Run: Apply rules
		TArray<FConcertPropertyChain> FoundOptions;
		Rules.AddDefaultPropertiesFromSettings(*USceneComponent::StaticClass(), [&FoundOptions](FConcertPropertyChain&& Chain)
		{
			FoundOptions.Emplace(MoveTemp(Chain));
		});

		// 3. Test
		TestEqual(TEXT("Num Found == Num Expected"), FoundOptions.Num(), PropertySelection.DefaultSelectedProperties.Num());
		// Everything that was in the rules was found ...
		for (const FString& Expected : PropertySelection.DefaultSelectedProperties)
		{
			const bool bFound = FoundOptions.ContainsByPredicate([&Expected](const FConcertPropertyChain& PropertyChain)
			{
				return Expected == PropertyChain.ToString();
			});
			TestTrue(FString::Printf(TEXT("Expected to match %s"), *Expected), bFound);
		}
		// ... and did not match anything not specified in the rules
		for (const FConcertPropertyChain& MatchedChain : FoundOptions)
		{
			const bool bFound = PropertySelection.DefaultSelectedProperties.ContainsByPredicate([&MatchedChain](const FString& Expected)
			{
				return MatchedChain.ToString() == Expected;
			});
			TestTrue(FString::Printf(TEXT("Found %s which was not specified by rules"), *MatchedChain.ToString()), bFound);
		}
		
		return true;
	}
}
