// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Data/ReplicationFrequencySettings.h"
#include "Replication/Processing/Proxy/FrequencyTracker.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"

namespace UE::ConcertSyncTests
{
	/**
	 * Tests that FFrequencyTracker works as expected:
	 *	- Tracks objects correctly,
	 *	- Does not leak objects when cleaning cache
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFrequencyTrackerTest, "VirtualProduction.Concert.Replication.Components.FrequencyTracker", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FFrequencyTrackerTest::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncCore;
		class FFrequencyTrackerMock : public FFrequencyTracker_CleanByTick
		{
		public:

			FFrequencyTrackerMock(const double CacheRetentionSeconds)
				: FFrequencyTracker_CleanByTick(CacheRetentionSeconds)
			{}
			
			using FFrequencyTracker_CleanByTick::GetObjectsToLastProcessed;
		};

		// 1. Init test data
		const FSoftObjectPath ReplicatedObject = GetMutableDefault<UObject>();
		const FConcertObjectInStreamID SpecifiedRateObject { FGuid{ 1, 0, 0, 0 }, ReplicatedObject };
		const FConcertObjectInStreamID RealtimeObject { FGuid{ 2, 0, 0, 0 }, ReplicatedObject };

		constexpr FConcertObjectReplicationSettings SpecifiedRateSettings { EConcertObjectReplicationMode::SpecifiedRate, 30 };
		constexpr FConcertObjectReplicationSettings RealtimeSettings { EConcertObjectReplicationMode::Realtime };

		const double Now = FPlatformTime::Seconds();
		constexpr double CacheRetentionSeconds = 5.f;
		FFrequencyTrackerMock TestFrequencyTracker(CacheRetentionSeconds);

		// 2.1 Simulate replicating immediately
		TestEqual(TEXT("Can immediately replicate realtime object"),
			TestFrequencyTracker.TrackAndCheckObject(RealtimeObject, RealtimeSettings, Now),
			ECanProcessObject::CanProcess
			);
		TestEqual(TEXT("Can immediately replicate throttled object"),
			TestFrequencyTracker.TrackAndCheckObject(SpecifiedRateObject, SpecifiedRateSettings, Now),
			ECanProcessObject::CanProcess
			);
		TestFrequencyTracker.OnProcessObject(RealtimeObject, Now);
		TestFrequencyTracker.OnProcessObject(SpecifiedRateObject, Now);
		
		TestEqual(TEXT("Throttled needs to wait after sending"),
			TestFrequencyTracker.TrackAndCheckObject(SpecifiedRateObject, SpecifiedRateSettings, Now),
			ECanProcessObject::TooEarly
			);

		// 2.2 Simulate replicating after specified rate has elapsed
		const double AfterIntervalElapsed = Now + SpecifiedRateSettings.GetUpdateIntervalInSeconds();
		TestEqual(TEXT("Can still replicate realtime object"),
			TestFrequencyTracker.TrackAndCheckObject(RealtimeObject, RealtimeSettings, AfterIntervalElapsed),
			ECanProcessObject::CanProcess
			);
		TestEqual(TEXT("Throttled object can send again"),
			TestFrequencyTracker.TrackAndCheckObject(SpecifiedRateObject, SpecifiedRateSettings, AfterIntervalElapsed),
			ECanProcessObject::CanProcess
			);

		// 2.3 Clean up cache
		constexpr double FakeWaitSeconds = CacheRetentionSeconds + 1.0;
		// Pretend that FakeWaitTime has elapsed so the cache is cleared.
		TestFrequencyTracker.Tick(FakeWaitSeconds, AfterIntervalElapsed + FakeWaitSeconds);
		TestEqual(TEXT("Cache was cleaned up after long time of not sending"),
			TestFrequencyTracker.GetObjectsToLastProcessed().Num(),
			0
			);

		return true;
	}
}
