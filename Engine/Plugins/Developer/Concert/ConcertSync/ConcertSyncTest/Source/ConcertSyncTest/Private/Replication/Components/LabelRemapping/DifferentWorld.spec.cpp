// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Misc/ActorLabelRemappingEditor.h"
#include "Replication/Util/ReplicatedTestWorld.h"

#include "Components/PointLightComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::ActorLabelRemapping
{
	/** This test that remapping works when the target world is different from the origin world. */
	BEGIN_DEFINE_SPEC(FActorLabelRemappingDifferentWorldsSpec, "VirtualProduction.Concert.Replication.Components.ActorLabelRemapping.DifferentWorlds", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FReplicatedTestWorld> OriginWorld;
		TUniquePtr<FReplicatedTestWorld> TargetWorld;

		UWorld& GetOriginWorld() const { return *OriginWorld->PreviewScene.GetWorld(); }
		UWorld& GetTargetWorld() const { return *TargetWorld->PreviewScene.GetWorld(); }
		
		/** Tests that only PropertyName is replicated by Object in Map */
		void TestReplicatesOnlyProperty(const FConcertObjectReplicationMap& Map, const UObject* Object, FName PropertyName)
		{
			Replication::TestReplicatesOnlyProperty(Map, Object, PropertyName, *this);
		}
	END_DEFINE_SPEC(FActorLabelRemappingDifferentWorldsSpec);

	void FActorLabelRemappingDifferentWorldsSpec::Define()
	{
		BeforeEach([this]
		{
			OriginWorld = MakeUnique<FReplicatedTestWorld>();
			TargetWorld = MakeUnique<FReplicatedTestWorld>();
		});
		AfterEach([this]
		{
			OriginWorld.Reset();
			TargetWorld.Reset();
		});
		
		It("Remap actor that retains label and object path", [this]
		{
			// 1. Setup objects
			AStaticMeshActor* OriginActor = OriginWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0_Origin"), TEXT("Label0"));
			UStaticMeshComponent* OriginComponent = OriginActor->GetStaticMeshComponent();
			OriginWorld->AddReplicatedProperty(OriginActor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			OriginWorld->AddReplicatedProperty(OriginComponent, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));

			AStaticMeshActor* TargetActor = TargetWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0_Target"), TEXT("Label0"));
			UStaticMeshComponent* TargetComponent = TargetActor->GetStaticMeshComponent();
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(OriginWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(OriginWorld->ReplicationMap, RemappingData, GetTargetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 2);
			TestReplicatesOnlyProperty(Translation, TargetActor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			TestReplicatesOnlyProperty(Translation, TargetComponent, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
		});

		It("Remap component of which the actor is not replicated", [this]
		{
			// 1. Setup objects
			AStaticMeshActor* OriginActor = OriginWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0_Origin"), TEXT("Label0"));
			UStaticMeshComponent* OriginComponent = OriginActor->GetStaticMeshComponent();
			OriginWorld->AddReplicatedProperty(OriginComponent, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			AStaticMeshActor* TargetActor = TargetWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0_Target"), TEXT("Label0"));
			UStaticMeshComponent* TargetComponent = TargetActor->GetStaticMeshComponent();
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(OriginWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(OriginWorld->ReplicationMap, RemappingData, GetTargetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 1);
			TestReplicatesOnlyProperty(Translation, TargetComponent, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
		});

		It("Do not remap to actor that does not have the label", [this]
		{
			// To add potential risk, we'll make both actors have the same name.
			const FName ActorName(TEXT("StatisMeshActor0"));
			
			// 1. Setup objects
			AStaticMeshActor* OriginActor = OriginWorld->SpawnActor<AStaticMeshActor>(ActorName, TEXT("Label0"));
			UStaticMeshComponent* OriginComponent = OriginActor->GetStaticMeshComponent();
			OriginWorld->AddReplicatedProperty(OriginComponent, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			TargetWorld->SpawnActor<AStaticMeshActor>(ActorName, TEXT("Label0_Other"));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(OriginWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(OriginWorld->ReplicationMap, RemappingData, GetTargetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 0);
		});
		
		It("Correctly remap actors that swapped labels", [this]
		{
			// 1. Setup objects
			AStaticMeshActor* OriginCross0 = OriginWorld->SpawnActor<AStaticMeshActor>(TEXT("Cross0"), TEXT("Label0"));
			AStaticMeshActor* OriginCross1 = OriginWorld->SpawnActor<AStaticMeshActor>(TEXT("Cross1"), TEXT("Label1"));
			UStaticMeshComponent* OriginComponent0 = OriginCross0->GetStaticMeshComponent();
			UStaticMeshComponent* OriginComponent1 = OriginCross1->GetStaticMeshComponent();
			OriginWorld->AddReplicatedProperty(OriginCross0, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			OriginWorld->AddReplicatedProperty(OriginComponent0, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			OriginWorld->AddReplicatedProperty(OriginCross1, GET_MEMBER_NAME_CHECKED(AActor, bOnlyRelevantToOwner));
			OriginWorld->AddReplicatedProperty(OriginComponent1, GET_MEMBER_NAME_CHECKED(USceneComponent, bHiddenInGame));
			
			AStaticMeshActor* TargetCross0 = TargetWorld->SpawnActor<AStaticMeshActor>(TEXT("Cross0"), TEXT("Label1"));
			AStaticMeshActor* TargetCross1 = TargetWorld->SpawnActor<AStaticMeshActor>(TEXT("Cross1"), TEXT("Label0"));
			UStaticMeshComponent* TargetComponent0 = TargetCross0->GetStaticMeshComponent();
			UStaticMeshComponent* TargetComponent1 = TargetCross1->GetStaticMeshComponent();
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(OriginWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(OriginWorld->ReplicationMap, RemappingData, GetTargetWorld());

			// 3. 
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 4);
			// ... swaps the assigned properties
			TestReplicatesOnlyProperty(Translation, TargetCross0, GET_MEMBER_NAME_CHECKED(AActor, bOnlyRelevantToOwner));
			TestReplicatesOnlyProperty(Translation, TargetComponent0, GET_MEMBER_NAME_CHECKED(USceneComponent, bHiddenInGame));
			TestReplicatesOnlyProperty(Translation, TargetCross1, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			TestReplicatesOnlyProperty(Translation, TargetComponent1, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
		});
		
		It("Actor is not remapped if owned, replicated component class changes", [this]
		{
			// 1. Setup objects
			const FName ComponentName(TEXT("Component"));
			AActor* OriginActor = OriginWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			UStaticMeshComponent* OriginComponent = NewObject<UStaticMeshComponent>(OriginActor, ComponentName);
			OriginActor->AddOwnedComponent(OriginComponent);
			OriginWorld->AddReplicatedProperty(OriginActor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			OriginWorld->AddReplicatedProperty(OriginComponent, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			AActor* TargetActor = TargetWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			UPointLightComponent* TargetComponent = NewObject<UPointLightComponent>(TargetActor, ComponentName);
			TargetActor->AddOwnedComponent(TargetComponent);
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(OriginWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(OriginWorld->ReplicationMap, RemappingData, GetTargetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 0);
		});
		It("Actor matched if owned, non-replicated component class changes", [this]
		{
			// 1. Setup objects
			const FName ComponentName(TEXT("Component"));
			AActor* OriginActor = OriginWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			UStaticMeshComponent* OriginComponent = NewObject<UStaticMeshComponent>(OriginActor, ComponentName);
			OriginActor->AddOwnedComponent(OriginComponent);
			OriginWorld->AddReplicatedProperty(OriginActor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			
			AActor* TargetActor = TargetWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			UPointLightComponent* TargetComponent = NewObject<UPointLightComponent>(TargetActor, ComponentName);
			TargetActor->AddOwnedComponent(TargetComponent);
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(OriginWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(OriginWorld->ReplicationMap, RemappingData, GetTargetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 1);
			TestReplicatesOnlyProperty(Translation, TargetActor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
		});

		It("Actor is not remapped if owned, replicated component name changes", [this]
		{
			// 1. Setup objects
			AActor* OriginActor = OriginWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			UStaticMeshComponent* OriginComponent = NewObject<UStaticMeshComponent>(OriginActor, TEXT("OriginComponent"));
			OriginActor->AddOwnedComponent(OriginComponent);
			OriginWorld->AddReplicatedProperty(OriginActor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			OriginWorld->AddReplicatedProperty(OriginComponent, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			AActor* TargetActor = TargetWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			UPointLightComponent* TargetComponent = NewObject<UPointLightComponent>(TargetActor, TEXT("TargetComponent"));
			TargetActor->AddOwnedComponent(TargetComponent);
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(OriginWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(OriginWorld->ReplicationMap, RemappingData, GetTargetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 0);
		});

		It("Cannot remap onto actor that has changed class but has same name and label", [this]
		{
			// 1. Setup objects
			AActor* OriginActor = OriginWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			OriginWorld->AddReplicatedProperty(OriginActor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			
			TargetWorld->SpawnActor<AStaticMeshActor>(TEXT("Actor0"), TEXT("Label0"));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(OriginWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(OriginWorld->ReplicationMap, RemappingData, GetTargetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 0);
		});
	}
}
