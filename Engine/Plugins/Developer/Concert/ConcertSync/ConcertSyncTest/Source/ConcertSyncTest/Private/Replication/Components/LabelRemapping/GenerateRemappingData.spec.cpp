// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Misc/ActorLabelRemappingEditor.h"
#include "Replication/Util/ReplicatedTestWorld.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::ActorLabelRemapping
{
	/** This test that remapping works after changes have been made to the world. */
	BEGIN_DEFINE_SPEC(FGenerateRemappingDataSpec, "VirtualProduction.Concert.Replication.Components.ActorLabelRemapping.GenerateRemappingDataSpec", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FReplicatedTestWorld> ReplicatedWorld;

		UWorld& GetWorld() const { return *ReplicatedWorld->PreviewScene.GetWorld(); }
		
		/** Tests that only PropertyName is replicated by Object in Map */
		void TestReplicatesOnlyProperty(const FConcertObjectReplicationMap& Map, const UObject* Object, FName PropertyName)
		{
			Replication::TestReplicatesOnlyProperty(Map, Object, PropertyName, *this);
		}
	END_DEFINE_SPEC(FGenerateRemappingDataSpec);
	
	void FGenerateRemappingDataSpec::Define()
	{
		BeforeEach([this]
		{
			ReplicatedWorld = MakeUnique<FReplicatedTestWorld>();
		});
		AfterEach([this]
		{
			ReplicatedWorld.Reset();
		});
		
		It("Contains actor data when actor is replicated", [this]
		{
			// 1. Create objects
			const FString ActorLabel = TEXT("Label0");
			AStaticMeshActor* Actor = ReplicatedWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0"), ActorLabel);
			ReplicatedWorld->AddReplicatedProperty(Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));

			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			const FConcertReplicationRemappingData_Actor* ActorData = RemappingData.ActorData.Find(Actor);
			
			// 3. Test data
			if (!ActorData)
			{
				AddError(TEXT("Missing data"));
			}
			else
			{
				TestEqual(TEXT("Label"), ActorData->Label, ActorLabel);
				TestEqual(TEXT("Class"), ActorData->Class, FSoftClassPath(Actor->GetClass()));
			}
		});

		It("Contains actor data even if only component is replicated", [this]
		{
			// 1. Create objects
			const FString ActorLabel = TEXT("Label0");
			AStaticMeshActor* Actor = ReplicatedWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0"), ActorLabel);
			UStaticMeshComponent* Comp = Actor->GetStaticMeshComponent();
			ReplicatedWorld->AddReplicatedProperty(Comp, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			const FConcertReplicationRemappingData_Actor* ActorData = RemappingData.ActorData.Find(Actor);
			
			// 3. Test data
			if (!ActorData)
			{
				AddError(TEXT("Missing data"));
			}
			else
			{
				TestEqual(TEXT("Label"), ActorData->Label, ActorLabel);
				TestEqual(TEXT("Class"), ActorData->Class, FSoftClassPath(Actor->GetClass()));
			}
		});
		
		It("Contains no actor data if component's owning actor class cannot be gotten", [this]
		{
			// 1. Create objects
			const FString ActorLabel = TEXT("Label0");
			const FSoftObjectPath ActorPath(TEXT("/InternalTest/Map.Map:PersistentLevel.Actor"));
			const FSoftObjectPath ComponentPath(TEXT("/InternalTest/Map.Map:PersistentLevel.Actor.Component"));
			ensureAlways(ActorPath.ResolveObject() == nullptr);
			
			FConcertObjectReplicationMap ReplicationMap;
			FConcertReplicatedObjectInfo& Info = ReplicationMap.ReplicatedObjects.Add(ComponentPath);
			Info.ClassPath = USceneComponent::StaticClass();
			const TOptional<FConcertPropertyChain> Property = FConcertPropertyChain::CreateFromPath(
				*USceneComponent::StaticClass(),
				{ GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity) }
				);
			if (!Property)
			{
				AddError(TEXT("Failed get FConcertPropertyChain for property"));
				return;
			}
			Info.PropertySelection.ReplicatedProperties.Add(*Property);
			
			// 2. Generate data
			// Internally, GenerateRemappingData will try to resolve ActorPath but fail because the instance does not exist.
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			
			// 3. Test data
			TestFalse(TEXT("Does not contain data"), RemappingData.ActorData.Contains(ActorPath));
		});
	}
}
