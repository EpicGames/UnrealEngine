// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Misc/ActorLabelRemappingEditor.h"
#include "Replication/Util/ReplicatedTestWorld.h"

#include "Components/PointLightComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/BlockingVolume.h"
#include "Engine/StaticMeshActor.h"
#include "GameFramework/CameraBlockingVolume.h"
#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::ActorLabelRemapping
{
	/** This test that remapping works after changes have been made to the world. */
	BEGIN_DEFINE_SPEC(FActorLabelRemappingSameWorldSpec, "VirtualProduction.Concert.Replication.Components.ActorLabelRemapping.SameWorld", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FReplicatedTestWorld> ReplicatedWorld;

		UWorld& GetWorld() const { return *ReplicatedWorld->PreviewScene.GetWorld(); }
	
		/** Tests that only PropertyName is replicated by Object in Map */
		void TestReplicatesOnlyProperty(const FConcertObjectReplicationMap& Map, const UObject* Object, FName PropertyName)
		{
			Replication::TestReplicatesOnlyProperty(Map, Object, PropertyName, *this);
		}
	END_DEFINE_SPEC(FActorLabelRemappingSameWorldSpec);

	void FActorLabelRemappingSameWorldSpec::Define()
	{
		BeforeEach([this]
		{
			ReplicatedWorld = MakeUnique<FReplicatedTestWorld>();
		});
		AfterEach([this]
		{
			ReplicatedWorld.Reset();
		});

		It("Remap actor that retains label and object path", [this]
		{
			// 1. Setup objects
			AStaticMeshActor* Actor = ReplicatedWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0"), TEXT("Label0"));
			UStaticMeshComponent* Component = Actor->GetStaticMeshComponent();
			ReplicatedWorld->AddReplicatedProperty(Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			ReplicatedWorld->AddReplicatedProperty(Component, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(ReplicatedWorld->ReplicationMap, RemappingData, GetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 2);
			TestReplicatesOnlyProperty(Translation, Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			TestReplicatesOnlyProperty(Translation, Component, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
		});

		It("Remap component of which the actor is not replicated", [this]
		{
			// 1. Setup objects
			AStaticMeshActor* Actor = ReplicatedWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0"), TEXT("Label0"));
			UStaticMeshComponent* Component = Actor->GetStaticMeshComponent();
			ReplicatedWorld->AddReplicatedProperty(Component, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(ReplicatedWorld->ReplicationMap, RemappingData, GetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 1);
			TestReplicatesOnlyProperty(Translation, Component, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
		});

		It("Do not remap to actor that no longer has the label", [this]
		{
			// 1. Setup objects
			AStaticMeshActor* Actor = ReplicatedWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0"), TEXT("Label0"));
			ReplicatedWorld->AddReplicatedProperty(Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			Actor->SetActorLabel(TEXT("ChangedLabel"));
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(ReplicatedWorld->ReplicationMap, RemappingData, GetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 0);
		});
		
		It("Correctly remap actors that swapped labels", [this]
		{
			// 1. Setup objects
			AStaticMeshActor* Cross0 = ReplicatedWorld->SpawnActor<AStaticMeshActor>(TEXT("Cross0"), TEXT("Label0"));
			AStaticMeshActor* Cross1 = ReplicatedWorld->SpawnActor<AStaticMeshActor>(TEXT("Cross1"), TEXT("Label1"));
			UStaticMeshComponent* Component0 = Cross0->GetStaticMeshComponent();
			UStaticMeshComponent* Component1 = Cross1->GetStaticMeshComponent();
			ReplicatedWorld->AddReplicatedProperty(Cross0, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			ReplicatedWorld->AddReplicatedProperty(Component0, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			ReplicatedWorld->AddReplicatedProperty(Cross1, GET_MEMBER_NAME_CHECKED(AActor, bOnlyRelevantToOwner));
			ReplicatedWorld->AddReplicatedProperty(Component1, GET_MEMBER_NAME_CHECKED(USceneComponent, bHiddenInGame));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			// Swapping labels ...
			Cross0->SetActorLabel(TEXT("Label1"));
			Cross1->SetActorLabel(TEXT("Label0"));
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(ReplicatedWorld->ReplicationMap, RemappingData, GetWorld());

			// 3. 
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 4);
			// ... swaps the assigned properties
			TestReplicatesOnlyProperty(Translation, Cross0, GET_MEMBER_NAME_CHECKED(AActor, bOnlyRelevantToOwner));
			TestReplicatesOnlyProperty(Translation, Component0, GET_MEMBER_NAME_CHECKED(USceneComponent, bHiddenInGame));
			TestReplicatesOnlyProperty(Translation, Cross1, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			TestReplicatesOnlyProperty(Translation, Component1, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
		});

		It("Actor is not remapped if owned, replicated component class changes", [this]
		{
			// 1. Setup objects
			const FName ComponentName(TEXT("Component"));
			AActor* Actor = ReplicatedWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			UStaticMeshComponent* PreComponent = NewObject<UStaticMeshComponent>(Actor, ComponentName);
			Actor->AddOwnedComponent(PreComponent);
			ReplicatedWorld->AddReplicatedProperty(Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			ReplicatedWorld->AddReplicatedProperty(PreComponent, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			
			PreComponent->Rename(TEXT("TRASH"));
			UPointLightComponent* PostComponent = NewObject<UPointLightComponent>(Actor, ComponentName);
			Actor->AddOwnedComponent(PostComponent);
			
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(ReplicatedWorld->ReplicationMap, RemappingData, GetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 0);
		});
		It("Actor matched if owned, non-replicated component class changes", [this]
		{
			// 1. Setup objects
			const FName ComponentName(TEXT("Component"));
			AActor* Actor = ReplicatedWorld->SpawnActor<AActor>(TEXT("Actor0"), TEXT("Label0"));
			UStaticMeshComponent* PreComponent = NewObject<UStaticMeshComponent>(Actor, ComponentName);
			Actor->AddOwnedComponent(PreComponent);
			ReplicatedWorld->AddReplicatedProperty(Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
				
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			
			PreComponent->Rename(TEXT("TRASH"));
			UPointLightComponent* PostComponent = NewObject<UPointLightComponent>(Actor, ComponentName);
			Actor->AddOwnedComponent(PostComponent);
			
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(ReplicatedWorld->ReplicationMap, RemappingData, GetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 1);
			TestReplicatesOnlyProperty(Translation, Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
		});

		It("Actor is not remapped if owned, replicated component name changes", [this]
		{
			// 1. Setup objects
			AStaticMeshActor* Actor = ReplicatedWorld->SpawnActor<AStaticMeshActor>(TEXT("StaticMeshActor0"), TEXT("Label0"));
			UStaticMeshComponent* Component = Actor->GetStaticMeshComponent();
			ReplicatedWorld->AddReplicatedProperty(Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
			ReplicatedWorld->AddReplicatedProperty(Component, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
			
			// 2. Generate data
			const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
			Component->Rename(TEXT("NewComponentName"));
			const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(ReplicatedWorld->ReplicationMap, RemappingData, GetWorld());

			// 3. Test
			TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 0);
		});

		Describe("Cannot remap onto actor that has changed class but has same name and label", [this]
		{
			const auto RunVariation = [this](bool bActorHasProperties, bool bCreateComponent)
			{
				const FString Label(TEXT("Label0"));
				const FName ActorName(TEXT("Actor0"));
				const FName ComponentName(TEXT("Component"));
				
				// 1. Setup objects: ACameraBlockingVolume and ABlockingVolume were chosen because their classes have the same component hierarchy
				// to avoid a potential, unlikely point of failure for the test set up. Added bonus: they don't inherit from each other.
				AVolume* Actor = ReplicatedWorld->SpawnActor<ABlockingVolume>(ActorName, Label);
				if (bActorHasProperties)
				{
					ReplicatedWorld->AddReplicatedProperty(Actor, GET_MEMBER_NAME_CHECKED(AActor, bNetTemporary));
				}
				if (bCreateComponent)
				{
					UStaticMeshComponent* Component = NewObject<UStaticMeshComponent>(Actor, ComponentName);
					Actor->AddOwnedComponent(Component);
					ReplicatedWorld->AddReplicatedProperty(Component, GET_MEMBER_NAME_CHECKED(USceneComponent, ComponentVelocity));
				}
				
				// 2. Generate data
				const FConcertReplicationRemappingData RemappingData = ConcertSyncCore::GenerateRemappingData(ReplicatedWorld->ReplicationMap);
				Actor->Rename(TEXT("TRASH"));
				// Avoid the old actor being considered for remapping.
				Actor->SetActorLabel(TEXT("TRASH"));
				Actor->Destroy();
				Actor = ReplicatedWorld->SpawnActor<ACameraBlockingVolume>(ActorName, Label);
				if (bCreateComponent)
				{
					UStaticMeshComponent* Component = NewObject<UStaticMeshComponent>(Actor, ComponentName);
					Actor->AddOwnedComponent(Component);
				}
				const FConcertObjectReplicationMap Translation = ConcertSyncCore::RemapReplicationMap(ReplicatedWorld->ReplicationMap, RemappingData, GetWorld());
				
				// 3. Test
				TestEqual(TEXT("ReplicatedObjects.Num()"), Translation.ReplicatedObjects.Num(), 0);
			};

			It("Only actor has properties", [RunVariation] { RunVariation(true, false); });
			It("Only component has properties", [RunVariation] { RunVariation(false, true); });
			It("Actor and component have properties", [RunVariation] { RunVariation(true, true); });
		});
	}
}
