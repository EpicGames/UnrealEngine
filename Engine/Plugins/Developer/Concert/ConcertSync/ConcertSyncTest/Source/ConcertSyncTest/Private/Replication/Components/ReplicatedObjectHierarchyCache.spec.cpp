// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Replication/Misc/ReplicatedObjectHierarchyCache.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"

namespace UE::ConcertSyncTests::Replication
{
	/** Tests functions in ObjectPathUtils.h */
	BEGIN_DEFINE_SPEC(FReplicatedObjectHierarchyCacheSpec, "VirtualProduction.Concert.Replication.Components.ReplicatedObjectHierarchyCache", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		FGuid ClientId_One = FGuid::NewGuid(); 
		FGuid ClientId_Two = FGuid::NewGuid(); 
		FGuid ClientId_Three = FGuid::NewGuid(); 
	
		ConcertSyncCore::FReplicatedObjectHierarchyCache Cache;
		TSharedPtr<FObjectTestReplicator> ObjectReplicator;
		TSharedPtr<FObjectTestReplicator> SubobjectReplicator;

		void TestObjectInCache()
		{
			TestFalse(TEXT("IsObjectReferencedDirectly(Package)"), Cache.IsObjectReferencedDirectly(ObjectReplicator->TestObject->GetOuter()));
			TestFalse(TEXT("IsInHierarchy(Package)"), Cache.IsInHierarchy(ObjectReplicator->TestObject->GetOuter()).IsSet());
			
			TestTrue(TEXT("IsObjectReferencedDirectly(Object)"), Cache.IsObjectReferencedDirectly(ObjectReplicator->TestObject));
			TestTrue(TEXT("IsInHierarchy(Object)"), Cache.IsInHierarchy(ObjectReplicator->TestObject).IsSet());
		}
	
		void TestSubobjectInCache()
		{
			TestFalse(TEXT("IsObjectReferencedDirectly(Package)"), Cache.IsObjectReferencedDirectly(ObjectReplicator->TestObject->GetOuter()));
			TestFalse(TEXT("IsInHierarchy(Package)"), Cache.IsInHierarchy(ObjectReplicator->TestObject->GetOuter()).IsSet());
			
			TestTrue(TEXT("IsObjectReferencedDirectly(Subobject)"), Cache.IsObjectReferencedDirectly(SubobjectReplicator->TestObject));
			TestTrue(TEXT("IsInHierarchy(Subobject)"), Cache.IsInHierarchy(SubobjectReplicator->TestObject).IsSet());
		}
	END_DEFINE_SPEC(FReplicatedObjectHierarchyCacheSpec);

	void FReplicatedObjectHierarchyCacheSpec::Define()
	{
		BeforeEach([this]()
		{
			ObjectReplicator = MakeShared<FObjectTestReplicator>();
			SubobjectReplicator = ObjectReplicator->CreateSubobjectReplicator();
		});
		AfterEach([this]()
		{
			Cache.Clear();
			ObjectReplicator.Reset();
			SubobjectReplicator.Reset();
		});
		
		Describe("Single client", [this]()
		{
			It("OnJoin", [this]()
			{
				FConcertReplication_Join_Request JoinRequest;
				JoinRequest.Streams.Add(ObjectReplicator->CreateStream());
				
				Cache.OnJoin(ClientId_One, JoinRequest);
				
				TestObjectInCache();
			});

			It("OnChangeStreams", [this]()
			{
				const FConcertReplicationStream JoinStream = ObjectReplicator->CreateStream();
				FConcertReplication_Join_Request JoinRequest;
				JoinRequest.Streams.Add(JoinStream);
				
				Cache.OnJoin(ClientId_One, JoinRequest);
				Cache.OnChangeStreams(ClientId_One, { { FGuid::NewGuid(), SubobjectReplicator->TestObject } } , {});

				TestObjectInCache();
				TestSubobjectInCache();
			});
			
			It("OnPreClientLeft", [this]()
			{
				const FConcertReplicationStream JoinStream = ObjectReplicator->CreateStream();
				FConcertReplication_Join_Request JoinRequest;
				JoinRequest.Streams.Add(JoinStream);
				
				Cache.OnJoin(ClientId_One, JoinRequest);
				Cache.OnPostClientLeft(ClientId_One, { JoinStream });
				
				TestTrue(TEXT("IsEmpty"), Cache.IsEmpty());
			});
		});

		Describe("Two clients", [this]()
		{
			It("Object retained when one client leaves", [this]()
			{
				const FConcertReplicationStream JoinStream = ObjectReplicator->CreateStream();
				FConcertReplication_Join_Request JoinRequest;
				JoinRequest.Streams.Add(JoinStream);
				
				Cache.OnJoin(ClientId_One, JoinRequest);
				Cache.OnJoin(ClientId_Two, JoinRequest);
				Cache.OnPostClientLeft(ClientId_One, { JoinStream });
				
				TestObjectInCache();
			});

			It("Object is removed when both clients leave", [this]()
			{
				const FConcertReplicationStream JoinStream = ObjectReplicator->CreateStream();
				FConcertReplication_Join_Request JoinRequest;
				JoinRequest.Streams.Add(JoinStream);
				
				Cache.OnJoin(ClientId_One, JoinRequest);
				Cache.OnJoin(ClientId_Two, JoinRequest);
				Cache.OnPostClientLeft(ClientId_One, { JoinStream });
				Cache.OnPostClientLeft(ClientId_Two, { JoinStream });
				
				TestTrue(TEXT("IsEmpty"), Cache.IsEmpty());
			});
		});

		It("IsObjectReferencedDirectly with ignored clients", [this]()
		{
			FConcertReplication_Join_Request JoinRequest;
			JoinRequest.Streams.Add(ObjectReplicator->CreateStream());
				
			Cache.OnJoin(ClientId_One, JoinRequest);
			Cache.OnJoin(ClientId_Two, JoinRequest);
			Cache.OnJoin(ClientId_Three, JoinRequest);

			const FGuid IgnoreOneTwo[] = { ClientId_One, ClientId_Two };
			TestTrue(TEXT("IsObjectReferencedDirectly (ignore clients 1 and 2)"), Cache.IsObjectReferencedDirectly(ObjectReplicator->TestObject, IgnoreOneTwo));
			const FGuid IgnoreAll[] = { ClientId_One, ClientId_Two, ClientId_Three };
			TestTrue(TEXT("IsObjectReferencedDirectly (ignore all)"), !Cache.IsObjectReferencedDirectly(ObjectReplicator->TestObject, IgnoreAll));
		});
	}
}
