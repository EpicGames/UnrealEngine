// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Replication/Formats/IObjectReplicationFormat.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Replication/Processing/ObjectReplicationCache.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/TestReflectionObject.h"
#include "Util/ClientServerCommunicationTest.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

namespace UE::ConcertSyncTests
{
	/** Fake format that just sends around FNativeStruct and combines the contained float value by addition; no real implementation would actually do that but this is easier for testing. */
	class FTestReplicationFormat : public ConcertSyncCore::IObjectReplicationFormat
	{
	public:
		virtual TOptional<FConcertSessionSerializedPayload> CreateReplicationEvent(UObject& Object, FAllowPropertyFunc IsPropertyAllowedFunc) override { return NotMocked<TOptional<FConcertSessionSerializedPayload>>({}); }
		virtual void ClearInternalCache(TArrayView<UObject> ObjectsToClear) override { return NotMocked<void>(); }
		virtual void CombineReplicationEvents(FConcertSessionSerializedPayload& Base, const FConcertSessionSerializedPayload& Newer) override
		{
			// Reuse FNativeStruct to avoid introducing more test types
			FNativeStruct BaseStruct;
			Base.GetTypedPayload(BaseStruct);
			FNativeStruct NewerStruct;
			Newer.GetTypedPayload(NewerStruct);

			BaseStruct.Float += NewerStruct.Float;
			Base.SetTypedPayload(BaseStruct);
		}
		virtual void ApplyReplicationEvent(UObject& Object, const FConcertSessionSerializedPayload& Payload, const FOnPropertyVisitedFunc& OnPrePropertySerialized) override
		{
			return NotMocked<void>();
		}

		static FConcertReplication_ObjectReplicationEvent CreateEvent(FSoftObjectPath Object, float Value)
		{
			FNativeStruct Data{ Value };
			FConcertSessionSerializedPayload Payload;
			Payload.SetTypedPayload(Data, EConcertPayloadCompressionType::None);
			return { MoveTemp(Object), -1, Payload };
		}
	};

	enum class EReplicationCacheTestFlags
	{
		NeverConsume = 0,
		NeverReceive = 1,
		ConsumeManually = 2
	};
	ENUM_CLASS_FLAGS(EReplicationCacheTestFlags);

	/** Mock implementation which does different things based on EReplicationCacheTestFlags. */
	class FTestReplicationCacheUser : public ConcertSyncCore::IReplicationCacheUser
	{
	public:

		FAutomationTestBase& Test;
		FConcertObjectInStreamID AllowedObject;
		EReplicationCacheTestFlags Flags;
		ConcertSyncCore::FSequenceId LastSequenceId = 100;
		TSharedPtr<const FConcertReplication_ObjectReplicationEvent> CachedData;

		bool bWasOnDataCachedCalled = false;
		bool bWasOnCachedDataUpdatedCalled = false;

		FTestReplicationCacheUser(FAutomationTestBase& Test, FConcertObjectInStreamID AllowedObject, EReplicationCacheTestFlags Flags)
			: Test(Test)
			, AllowedObject(MoveTemp(AllowedObject))
			, Flags(Flags)
		{}
		
		virtual bool WantsToAcceptObject(const FConcertReplicatedObjectId& Object) const override
		{
			return !EnumHasAnyFlags(Flags, EReplicationCacheTestFlags::NeverReceive) && AllowedObject == Object;
		}
		
		virtual void OnDataCached(const FConcertReplicatedObjectId& Object, ConcertSyncCore::FSequenceId SequenceId, TSharedRef<const FConcertReplication_ObjectReplicationEvent> Data) override
		{
			bWasOnDataCachedCalled = true;
			if (EnumHasAnyFlags(Flags, EReplicationCacheTestFlags::NeverReceive))
			{
				Test.AddError(TEXT("Received object that we never asked for!"));
				return;
			}
			
			LastSequenceId = SequenceId;
			CachedData = Data;
		}

		virtual void OnCachedDataUpdated(const FConcertReplicatedObjectId& Object, ConcertSyncCore::FSequenceId SequenceId) override
		{
			bWasOnCachedDataUpdatedCalled = true;
			LastSequenceId = SequenceId;
		}

		float PeakData() const
		{
			if (!CachedData)
			{
				return -1.f;
			}

			FNativeStruct Data;
			CachedData->SerializedPayload.GetTypedPayload(Data);
			return Data.Float;
		}

		const void* GetDataAddress() const
		{
			return CachedData.Get();
		}
		
		void Consume()
		{
			check(!EnumHasAnyFlags(Flags, EReplicationCacheTestFlags::NeverConsume));
			CachedData.Reset();
		}

		void ResetCallFlags()
		{
			bWasOnDataCachedCalled = false;
			bWasOnCachedDataUpdatedCalled = false;
		}
	};
	
	BEGIN_DEFINE_SPEC(FReplicationCacheSpec, "VirtualProduction.Concert.Replication.Components.ReplicationCache", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		// Set up data
		const FGuid StreamId = FGuid::NewGuid();
		const FSoftObjectPath ObjectPath { TEXT("/Game/World.World:PersistentLevel.StaticMeshActor0") };
		const FGuid DummySendingClientId = FGuid::NewGuid();
		const FConcertReplicatedObjectId ObjectID{ { StreamId, ObjectPath }, DummySendingClientId};

		// Events that will be sent ...
		FConcertReplication_ObjectReplicationEvent Event_5;
		FConcertReplication_ObjectReplicationEvent Event_10;
		FConcertReplication_ObjectReplicationEvent Event_100;
		// ... and their sequence IDs
		const ConcertSyncCore::FSequenceId SequenceId_Event_5 = 0;
		const ConcertSyncCore::FSequenceId SequenceId_Event_10 = 1;
		const ConcertSyncCore::FSequenceId SequenceId_Event_100 = 2;

		// That cache that will be tested
		FTestReplicationFormat TestReplicationFormat;
		TSharedPtr<ConcertSyncCore::FObjectReplicationCache> Cache;
		
		// A bunch of users that have different implementations
		TSharedPtr<FTestReplicationCacheUser> User_NeverConsume;
		TSharedPtr<FTestReplicationCacheUser> User_NeverReceive;
		TSharedPtr<FTestReplicationCacheUser> User_ConsumeManually;
	END_DEFINE_SPEC(FReplicationCacheSpec);

	void FReplicationCacheSpec::Define()
	{
		BeforeEach([this]()
		{
			Event_5 = FTestReplicationFormat::CreateEvent(ObjectPath, 5.f);
			Event_10 = FTestReplicationFormat::CreateEvent(ObjectPath, 10.f);
			Event_100 = FTestReplicationFormat::CreateEvent(ObjectPath, 100.f);
			
			Cache					= MakeShared<ConcertSyncCore::FObjectReplicationCache>(TestReplicationFormat);
			
			User_NeverConsume		= MakeShared<FTestReplicationCacheUser>(*this, ObjectID, EReplicationCacheTestFlags::NeverConsume);
			User_NeverReceive		= MakeShared<FTestReplicationCacheUser>(*this, ObjectID, EReplicationCacheTestFlags::NeverReceive);
			User_ConsumeManually	= MakeShared<FTestReplicationCacheUser>(*this, ObjectID, EReplicationCacheTestFlags::ConsumeManually);

			Cache->RegisterDataCacheUser(User_NeverConsume.ToSharedRef());
			Cache->RegisterDataCacheUser(User_NeverReceive.ToSharedRef());
			Cache->RegisterDataCacheUser(User_ConsumeManually.ToSharedRef());
		});

		Describe("StoreUntilConsumed", [this]()
		{
			It("Calls OnDataCached if WantsToAcceptObject", [this]()
			{
				// 1st event
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_5, Event_5);
				
				// User_NeverConsume
				TestEqual(TEXT("User_NeverConsume: 5"), User_NeverConsume->PeakData(), 5.f);
				TestFalse(TEXT("User_NeverConsume: Didn't call OnCachedDataUpdated()"), User_NeverConsume->bWasOnCachedDataUpdatedCalled);
				TestTrue(TEXT("User_NeverConsume: Called OnDataCached()"), User_NeverConsume->bWasOnDataCachedCalled);
				TestEqual(TEXT("Sequence number: never consume"), User_NeverConsume->LastSequenceId, SequenceId_Event_5);

				// User_ConsumeManually
				TestEqual(TEXT("User_ConsumeManually: 5"), User_ConsumeManually->PeakData(), 5.f);
				TestFalse(TEXT("User_ConsumeManually: Didn't call OnCachedDataUpdated()"), User_ConsumeManually->bWasOnCachedDataUpdatedCalled);
				TestTrue(TEXT("User_ConsumeManually: Called OnDataCached()"), User_ConsumeManually->bWasOnDataCachedCalled);
				TestEqual(TEXT("Sequence number: manual consume"), User_ConsumeManually->LastSequenceId, SequenceId_Event_5);

				// User_NeverReceive
				TestFalse(TEXT("User_NeverReceive->bWasOnDataCachedCalled == false"), User_NeverReceive->bWasOnDataCachedCalled);
				TestFalse(TEXT("User_NeverReceive->bWasOnCachedDataUpdatedCalled == false"), User_NeverReceive->bWasOnCachedDataUpdatedCalled);
			});

			It("Combines data", [this]()
			{
				// 1st event
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_5, Event_5);
				// 2nd event
				User_NeverConsume->ResetCallFlags();
				User_ConsumeManually->ResetCallFlags();
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_10, Event_10);

				// User_NeverConsume
				TestEqual(TEXT("Combined events: 5 and 10"), User_NeverConsume->PeakData(), 15.f);
				TestTrue(TEXT("User_NeverConsume: Called OnCachedDataUpdated()"), User_NeverConsume->bWasOnCachedDataUpdatedCalled);
				TestFalse(TEXT("User_NeverConsume: Didn't call OnDataCached()"), User_NeverConsume->bWasOnDataCachedCalled);
				TestEqual(TEXT("Sequence number: never consume"), User_NeverConsume->LastSequenceId, SequenceId_Event_10);
			});

			It("Calls OnDataCached again if data was already consumed", [this]()
			{
				// 1st event
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_5, Event_5);

				TestEqual(TEXT("Received new data: 5"), User_ConsumeManually->PeakData(), 5.f);
				TestFalse(TEXT("User_ConsumeManually: Didn't call OnCachedDataUpdated()"), User_ConsumeManually->bWasOnCachedDataUpdatedCalled);
				TestTrue(TEXT("User_ConsumeManually: Called OnDataCached()"), User_ConsumeManually->bWasOnDataCachedCalled);
				TestEqual(TEXT("Sequence number: 5"), User_ConsumeManually->LastSequenceId, SequenceId_Event_5);
				
				// 2nd event
				User_ConsumeManually->Consume();
				User_ConsumeManually->ResetCallFlags();
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_10, Event_10);

				// User_ConsumeManually
				TestEqual(TEXT("Received new data: 10"), User_ConsumeManually->PeakData(), 10.f);
				TestFalse(TEXT("User_ConsumeManually: Didn't call OnCachedDataUpdated() again"), User_ConsumeManually->bWasOnCachedDataUpdatedCalled);
				TestTrue(TEXT("User_ConsumeManually: Called OnDataCached() again"), User_ConsumeManually->bWasOnDataCachedCalled);
				TestEqual(TEXT("Sequence number: 10"), User_ConsumeManually->LastSequenceId, SequenceId_Event_10);
			});

			It("Calls OnCachedDataUpdated if data was not yet consumed", [this]()
			{
				// 1st event
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_5, Event_5);

				TestEqual(TEXT("Received new data: 5"), User_ConsumeManually->PeakData(), 5.f);
				TestFalse(TEXT("User_ConsumeManually: Didn't call OnCachedDataUpdated()"), User_ConsumeManually->bWasOnCachedDataUpdatedCalled);
				TestTrue(TEXT("User_ConsumeManually: Called OnDataCached()"), User_ConsumeManually->bWasOnDataCachedCalled);
				TestEqual(TEXT("Sequence number: 5"), User_ConsumeManually->LastSequenceId, SequenceId_Event_5);
				
				// 2nd event
				User_ConsumeManually->ResetCallFlags();
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_10, Event_10);

				// User_ConsumeManually
				TestEqual(TEXT("Received new data: 15"), User_ConsumeManually->PeakData(), 15.f);
				TestTrue(TEXT("User_ConsumeManually: Called OnCachedDataUpdated()"), User_ConsumeManually->bWasOnCachedDataUpdatedCalled);
				TestFalse(TEXT("User_ConsumeManually: Didn't call OnDataCached()"), User_ConsumeManually->bWasOnDataCachedCalled);
				TestEqual(TEXT("Sequence number: 10"), User_ConsumeManually->LastSequenceId, SequenceId_Event_10);
			});
		});

		Describe("Cache does not leak data", [this]()
		{
			It("When unregistering users", [this]()
			{
				// 1st event
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_5, Event_5);
				User_ConsumeManually->Consume();
				
				// 2nd event
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_10, Event_10);
				Cache->UnregisterDataCacheUser(User_ConsumeManually.ToSharedRef());
				Cache->RegisterDataCacheUser(User_ConsumeManually.ToSharedRef());
				const void* AddressBefore = User_ConsumeManually->GetDataAddress();
				
				// 3rd event
				User_NeverConsume->ResetCallFlags();
				User_ConsumeManually->ResetCallFlags();
				Cache->StoreUntilConsumed(DummySendingClientId, StreamId, SequenceId_Event_100, Event_100);
				
				// User_ConsumeManually
				TestEqual(TEXT("Re-registered user only has the new data"), User_ConsumeManually->PeakData(), 100.f);
				TestNotEqual(TEXT("Re-registered user's data was allocated in different memory block"), AddressBefore, User_ConsumeManually->GetDataAddress());
				TestFalse(TEXT("User_ConsumeManually: Didn't call OnCachedDataUpdated()"), User_ConsumeManually->bWasOnCachedDataUpdatedCalled);
				TestTrue(TEXT("User_ConsumeManually: Called OnDataCached()"), User_ConsumeManually->bWasOnDataCachedCalled);
				TestEqual(TEXT("Sequence number: manual consume"), User_ConsumeManually->LastSequenceId, SequenceId_Event_100);
			});
		});
	}
}
