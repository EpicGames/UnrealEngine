// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Data/ObjectReplicationMap.h"

#include "Misc/AutomationTest.h"
#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Replication/Messages/ChangeStream.h"

/**
 * Tests for UE::ConcertSyncCore::Replication::ChangeStreamUtils
 */
namespace UE::ConcertSyncTests::ChangeStreamUtils
{
	/**
	 * Tests that UE::ConcertSyncCore::Replication::ChangeStreamUtils::DiffChanges detects
	 *  - adding and removing objects
	 *  - modify properties and class
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSimpleDiffChangesTest, "VirtualProduction.Concert.Replication.Components.DiffChangesSimple", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FSimpleDiffChangesTest::RunTest(const FString& Parameters)
	{
		// 1. Set up
		const FGuid StreamId = FGuid::NewGuid();
		const FSoftClassPath SceneComponentClass = USceneComponent::StaticClass();
		const FSoftClassPath StaticMeshComponentClass = UStaticMeshComponent::StaticClass();
		const FSoftObjectPath ChangeProperties(TEXT("/Game/World.World:PersistentLevel.ChangeProperties.StaticMeshComponent"));
		const FSoftObjectPath ChangeClass(TEXT("/Game/World.World:PersistentLevel.ChangeClass.StaticMeshComponent"));
		const FSoftObjectPath RemoveMe(TEXT("/Game/World.World:PersistentLevel.RemoveMe.StaticMeshComponent"));
		const FSoftObjectPath AddMe(TEXT("/Game/World.World:PersistentLevel.AddMe.StaticMeshComponent"));

		const TOptional<FConcertPropertyChain> RelativeLocation = FConcertPropertyChain::CreateFromPath(*USceneComponent::StaticClass(), { TEXT("RelativeLocation") });
		const TOptional<FConcertPropertyChain> RelativeLocationX = FConcertPropertyChain::CreateFromPath(*USceneComponent::StaticClass(), { TEXT("RelativeLocation"), TEXT("X") });
		const TOptional<FConcertPropertyChain> RelativeLocationY = FConcertPropertyChain::CreateFromPath(*USceneComponent::StaticClass(), { TEXT("RelativeLocation"), TEXT("Y") });
		const TOptional<FConcertPropertyChain> RelativeLocationZ = FConcertPropertyChain::CreateFromPath(*USceneComponent::StaticClass(), { TEXT("RelativeLocation"), TEXT("Z") });
		if (!ensureAlways(RelativeLocation && RelativeLocationX && RelativeLocationY && RelativeLocationZ))
		{
			return false;
		}
		const FConcertPropertySelection Selection_RelativeLocation { { *RelativeLocation, *RelativeLocationX, *RelativeLocationY, *RelativeLocationZ } };
		const FConcertPropertySelection Selection_RelativeLocationOnlyX { { *RelativeLocation, *RelativeLocationX} };
		

		// 2. Run
		FConcertObjectReplicationMap Base;
		Base.ReplicatedObjects.Add(ChangeProperties, { SceneComponentClass, Selection_RelativeLocation });
		Base.ReplicatedObjects.Add(ChangeClass, { SceneComponentClass, Selection_RelativeLocation });
		Base.ReplicatedObjects.Add(RemoveMe, { SceneComponentClass, Selection_RelativeLocation });
		FConcertObjectReplicationMap Desired;
		Desired.ReplicatedObjects.Add(ChangeProperties, { SceneComponentClass, Selection_RelativeLocationOnlyX });
		Desired.ReplicatedObjects.Add(ChangeClass, { StaticMeshComponentClass, Selection_RelativeLocation });
		Desired.ReplicatedObjects.Add(AddMe, { SceneComponentClass, Selection_RelativeLocation });
		const FConcertReplication_ChangeStream_Request Request = ConcertSyncCore::Replication::ChangeStreamUtils::BuildRequestFromDiff(StreamId, Base, Desired);

		
		// 3. Test
		constexpr int32 NumExpectedPutObjects = 3; // ChangeProperties + ChangeClass + AddMe
		TestEqual(TEXT("3 objects were put"), Request.ObjectsToPut.Num(), NumExpectedPutObjects);
		TestEqual(TEXT("No added streams"), Request.StreamsToAdd.Num(), 0);
		TestEqual(TEXT("No removed streams"), Request.StreamsToRemove.Num(), 0);
		
		TestEqual(TEXT("Remove exactly 1"), Request.ObjectsToRemove.Num(), 1);
		TestTrue(TEXT("Remove correct object"), Request.ObjectsToRemove.Contains({ StreamId, RemoveMe }));

		if (const FConcertReplication_ChangeStream_PutObject* ChangePropertiesChange = Request.ObjectsToPut.Find({ StreamId, ChangeProperties }))
		{
			// No class change, so it should be null in the request
			TestTrue(TEXT("Change Properties > Class unset"), ChangePropertiesChange->ClassPath.IsNull());
			TestEqual(TEXT("Change Properties > Properties"), ChangePropertiesChange->Properties, Selection_RelativeLocationOnlyX);
		}
		else
		{
			AddError(TEXT("Did not change ChangeProperties object"));
		}

		if (const FConcertReplication_ChangeStream_PutObject* ChangeClassChange = Request.ObjectsToPut.Find({ StreamId, ChangeClass }))
		{
			TestEqual(TEXT("Change Class > Class"), ChangeClassChange->ClassPath, StaticMeshComponentClass);
			// No properties changed, so it should be empty in the request
			TestTrue(TEXT("Change Class > Properties empty"), ChangeClassChange->Properties.ReplicatedProperties.IsEmpty());
		}
		else
		{
			AddError(TEXT("Did not change ChangeClass object"));
		}

		if (const FConcertReplication_ChangeStream_PutObject* AddMeChange = Request.ObjectsToPut.Find({ StreamId, AddMe }))
		{
			TestEqual(TEXT("Add Object > Class"), AddMeChange->ClassPath, SceneComponentClass);
			TestEqual(TEXT("Add Object > Properties"), AddMeChange->Properties, Selection_RelativeLocation);
		}
		else
		{
			AddError(TEXT("Did not add AddMe object"));
		}
		
		return true;
	}
}