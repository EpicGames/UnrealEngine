// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Replication/SyncControlState.h"
#include "Replication/Messages/SyncControl.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	BEGIN_DEFINE_SPEC(FSyncControlStateSpec, "VirtualProduction.Concert.Replication.Components", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	FGuid SenderStreamId = FGuid::NewGuid();
	/** Leverage to create a real FSoftObjectPath. */
	TSharedPtr<FObjectTestReplicator> ObjectReplicator;
	TSharedPtr<FObjectTestReplicator> SubobjectReplicator;
	END_DEFINE_SPEC(FSyncControlStateSpec);

	/** This tests that FSyncControlState correctly analyses requests and responses for aggregation. */
	void FSyncControlStateSpec::Define()
	{
		using namespace ConcertSyncCore::Replication;
		
		BeforeEach([this]
		{
			ObjectReplicator = MakeShared<FObjectTestReplicator>();
			SubobjectReplicator = ObjectReplicator->CreateSubobjectReplicator();
		});
		AfterEach([this]
		{
			// Test would hold onto this for rest of engine lifetime. Clean up this mini would-be leak.
			ObjectReplicator.Reset();
			SubobjectReplicator.Reset();
		});

		// At this point, everything should work server-side. Now test client-side prediction.
		Describe("Changing sync control state", [this]()
		{
			It("Is correct with releasing authority with FConcertReplication_ChangeAuthority_Request", [this]()
			{
				FSyncControlState SyncControl = TSet<FConcertObjectInStreamID>{{ SenderStreamId, ObjectReplicator->TestObject }};
				
				const FConcertReplication_ChangeAuthority_Request Request { .ReleaseAuthority = { { ObjectReplicator->TestObject, {{ SenderStreamId }} } } };
				const FConcertReplication_ChangeAuthority_Response Response;
				TArray<FConcertObjectInStreamID> RemovedObjects;
				SyncControl.AppendAuthorityChange(
					Request,
					Response.SyncControl,
					[this](const FConcertObjectInStreamID&){ AddError(TEXT("No object should be added")); },
					[&RemovedObjects](const FConcertObjectInStreamID& Object){ RemovedObjects.Add(Object); }
					);
				
				TestTrue(TEXT("Removed TestObject"), RemovedObjects.Contains(FConcertObjectInStreamID{ SenderStreamId, ObjectReplicator->TestObject }));
				TestEqual(TEXT("Removed exactly 1 object"), RemovedObjects.Num(), 1);
			});
			
			It("Is correct with taking authority with FConcertReplication_ChangeAuthority_Request", [this]()
			{
				FSyncControlState SyncControl;
				
				const FConcertObjectInStreamID ObjectId{ SenderStreamId, ObjectReplicator->TestObject };
				const FConcertReplication_ChangeAuthority_Request Request { .TakeAuthority = {{ ObjectReplicator->TestObject, {{ SenderStreamId }}}}};
				const FConcertReplication_ChangeAuthority_Response Response { .SyncControl = {{{ ObjectId, true }}}};
				TArray<FConcertObjectInStreamID> AddedObjects;
				SyncControl.AppendAuthorityChange(
					Request,
					Response.SyncControl,
					[&AddedObjects](const FConcertObjectInStreamID& Object){ AddedObjects.Add(Object); },
					[this](const FConcertObjectInStreamID&){ AddError(TEXT("No object should be removed")); }
					);
				
				TestTrue(TEXT("Added TestObject"), AddedObjects.Contains(ObjectId));
				TestEqual(TEXT("Added exactly 1 object"), AddedObjects.Num(), 1);
			});

			It("Is correct after releasing authority with FConcertReplication_ChangeStream_Request", [this]()
			{
				FSyncControlState SyncControl = TSet<FConcertObjectInStreamID>{{ SenderStreamId, ObjectReplicator->TestObject }};
				
				FConcertReplication_ChangeStream_Request StreamChange;
				StreamChange.ObjectsToRemove.Add({ SenderStreamId, ObjectReplicator->TestObject });
				TArray<FConcertObjectInStreamID> RemovedObjects;
				SyncControl.AppendStreamChange(StreamChange, [&RemovedObjects](const FConcertObjectInStreamID& Object){ RemovedObjects.Add(Object); });
				
				TestTrue(TEXT("Removed TestObject"), RemovedObjects.Contains(FConcertObjectInStreamID{ SenderStreamId, ObjectReplicator->TestObject }));
				TestEqual(TEXT("Removed exactly 1 object"), RemovedObjects.Num(), 1);
			});

			Describe("With FConcertReplication_ChangeMuteState_Request", [this]()
			{
				It("Predicts for request muting single object", [this]()
				{
					FSyncControlState SyncControl = TSet<FConcertObjectInStreamID>{{ SenderStreamId, ObjectReplicator->TestObject }};
					
					FConcertReplication_ChangeMuteState_Request Request;
					Request.ObjectsToMute.Add(ObjectReplicator->TestObject, { EConcertReplicationMuteOption::OnlyObject });
					const FSyncControlState::FPredictedObjectRemoval Removal = SyncControl.PredictAndApplyMuteChanges(Request);
					
					TestEqual(TEXT("SyncControl.Num() == 0"), SyncControl.Num(), 0);
				});
				
				It("Applies succeeded response unmuting object", [this]()
				{
					FSyncControlState SyncControl;
					
					FConcertReplication_ChangeMuteState_Request Request;
					Request.ObjectsToUnmute.Add(ObjectReplicator->TestObject, { EConcertReplicationMuteOption::OnlyObject });
					const FSyncControlState::FPredictedObjectRemoval Removal = SyncControl.PredictAndApplyMuteChanges(Request);
					TestEqual(TEXT("SyncControl.Num() == 0"), SyncControl.Num(), 0);

					FConcertReplication_ChangeMuteState_Response Response { EConcertReplicationMuteErrorCode::Accepted };
					Response.SyncControl.NewControlStates.Add({ SenderStreamId, ObjectReplicator->TestObject }, true);
					SyncControl.ApplyOrRevertMuteResponse(Removal, Response);
					
					TestEqual(TEXT("SyncControl.Num() == 1"), SyncControl.Num(), 1);
					TestTrue(TEXT("Contains TestObject"), SyncControl.IsObjectAllowed({ SenderStreamId, ObjectReplicator->TestObject }));
				});
				
				It("Reverts failed response muting single object", [this]()
				{
					FSyncControlState SyncControl = TSet<FConcertObjectInStreamID>{{ SenderStreamId, ObjectReplicator->TestObject }};
					
					FConcertReplication_ChangeMuteState_Request Request;
					Request.ObjectsToMute.Add(ObjectReplicator->TestObject, { EConcertReplicationMuteOption::OnlyObject });
					const FSyncControlState::FPredictedObjectRemoval Removal = SyncControl.PredictAndApplyMuteChanges(Request);

					const FConcertReplication_ChangeMuteState_Response Response { EConcertReplicationMuteErrorCode::Timeout };
					SyncControl.ApplyOrRevertMuteResponse(Removal, Response);
					
					TestEqual(TEXT("SyncControl.Num() == 1"), SyncControl.Num(), 1);
					TestTrue(TEXT("Contains TestObject"), SyncControl.IsObjectAllowed({ SenderStreamId, ObjectReplicator->TestObject }));
				});
				
				It("Predicts for request muting object & subobject", [this]()
				{
					FSyncControlState SyncControl = TSet<FConcertObjectInStreamID>
					{
						{ SenderStreamId, ObjectReplicator->TestObject },
						{ SenderStreamId, SubobjectReplicator->TestObject }
					};
					
					FConcertReplication_ChangeMuteState_Request Request;
					Request.ObjectsToMute.Add(ObjectReplicator->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects });
					const FSyncControlState::FPredictedObjectRemoval Removal = SyncControl.PredictAndApplyMuteChanges(Request);
					
					TestEqual(TEXT("SyncControl.Num() == 0"), SyncControl.Num(), 0);
				});

				It("Applies implicitly unmuted objects", [this]()
				{
					FSyncControlState SyncControl;

					FConcertReplication_ChangeMuteState_Request Request;
					Request.ObjectsToUnmute.Add(ObjectReplicator->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects });
					const FSyncControlState::FPredictedObjectRemoval Removal = SyncControl.PredictAndApplyMuteChanges(Request);
					
					FConcertReplication_ChangeMuteState_Response Response { EConcertReplicationMuteErrorCode::Accepted };
					Response.SyncControl.NewControlStates =
					{
						{ { SenderStreamId, ObjectReplicator->TestObject }, true },
						{ { SenderStreamId, SubobjectReplicator->TestObject }, true },
					};
					SyncControl.ApplyOrRevertMuteResponse(Removal, Response);
					
					TestEqual(TEXT("SyncControl.Num() == 2"), SyncControl.Num(), 2);
					TestTrue(TEXT("Contains root"), SyncControl.IsObjectAllowed({ SenderStreamId, ObjectReplicator->TestObject }));
					TestTrue(TEXT("Contains subobject"), SyncControl.IsObjectAllowed({ SenderStreamId, SubobjectReplicator->TestObject }));
				});
			});
		});
	}
}
