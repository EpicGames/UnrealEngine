// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/PropertyChainUtils.h"
#include "Replication/ReplicationPropertyFilter.h"
#include "TestReflectionObject.h"

#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"

namespace UE::ConcertSyncTest::Replication::PropertyChain
{
	/**
	 * Tests that the FConcertPropertyChain constructor works as the documentation dictates:
	 *  - Root properties
	 *  - Array, set, map of
	 *		- "normal" structs, like FVector > lists sub-properties
	 *		- primitives and native structs > Path contains FConcertPropertyChain::InternalContainerPropertyValueName ("Value") at the end
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertPropertyChainTests, "VirtualProduction.Concert.Replication.Data.PropertyChainTests", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FConcertPropertyChainTests::RunTest(const FString& Parameters)
	{
		// The test is intentionally set up to make it it easy to set breakpoints albeit at the expense of increasing the amount of code

		// 1. Input
		UStruct* TestReplicationStruct = FTestReplicationStruct::StaticStruct();
		FProperty* ValueProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, Value));
		FProperty* VectorProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, Vector));
		FProperty* VectorXProp = CastField<FStructProperty>(VectorProp)->Struct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FVector, X));
		FProperty* NativeStructProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, NativeStruct));
		FProperty* NativeStructSubProp = CastField<FStructProperty>(NativeStructProp)->Struct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FNativeStruct, Float));
	
		FProperty* FloatArrayProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, FloatArray));
		FProperty* StringArrayProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, StringArray));
		FProperty* NativeStructArrayProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, NativeStructArray));
	
		FProperty* FloatSetProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, FloatSet));
		FProperty* StringSetProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, StringSet));
		FProperty* NativeStructSetProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, NativeStructSet));
	
		FProperty* StringToFloatProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, StringToFloat));
		FProperty* StringToVectorProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, StringToVector));
		FProperty* StringToNativeStructProp = TestReplicationStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, StringToNativeStruct));

		FArchiveSerializedPropertyChain VectorXChain;
		FArchiveSerializedPropertyChain NativeStructChain;
		FArchiveSerializedPropertyChain FloatArrayChain;
		FArchiveSerializedPropertyChain FloatSetArrayChain;
		FArchiveSerializedPropertyChain StringToFloatChain;
		FArchiveSerializedPropertyChain NativeStructArrayChain;
		FArchiveSerializedPropertyChain NativeStructSetChain;
		FArchiveSerializedPropertyChain StringToNativeStructChain;
		VectorXChain.PushProperty(VectorProp, false);
		NativeStructChain.PushProperty(NativeStructProp, false);
		FloatArrayChain.PushProperty(FloatArrayProp, false);
		FloatSetArrayChain.PushProperty(FloatSetProp, false);
		StringToFloatChain.PushProperty(StringToFloatProp, false);
		NativeStructArrayChain.PushProperty(NativeStructArrayProp, false);
		NativeStructSetChain.PushProperty(NativeStructSetProp, false);
		StringToNativeStructChain.PushProperty(StringToNativeStructProp, false);
	

	
		// 2. Run
		const FConcertPropertyChain Value(nullptr, *ValueProp);
		const FConcertPropertyChain Vector(nullptr, *VectorProp);
		const FConcertPropertyChain VectorX(&VectorXChain, *VectorXProp);
		const FConcertPropertyChain NativeStruct(nullptr, *NativeStructProp);
		const FConcertPropertyChain NativeStructSubProperty(&NativeStructChain, *NativeStructSubProp);
	
		const FConcertPropertyChain FloatArray(nullptr, *FloatArrayProp);
		const FConcertPropertyChain StringArray(nullptr, *StringArrayProp);
		const FConcertPropertyChain NativeStructArray(nullptr, *NativeStructArrayProp);
		const FConcertPropertyChain NativeStructArraySubProperty(&NativeStructArrayChain, *NativeStructSubProp);
	
		const FConcertPropertyChain FloatSet(nullptr, *FloatSetProp);
		const FConcertPropertyChain StringSet(nullptr, *StringSetProp);
		const FConcertPropertyChain NativeStructSet(nullptr, *NativeStructSetProp);
		const FConcertPropertyChain NativeStructSetSubProperty(&NativeStructSetChain, *NativeStructSubProp);
	
		const FConcertPropertyChain StringToFloat(nullptr, *StringToFloatProp);
		const FConcertPropertyChain StringToVector(nullptr, *StringToVectorProp);
		const FConcertPropertyChain StringToNativeStruct(nullptr, *StringToNativeStructProp);
		const FConcertPropertyChain StringToNativeStructSubProperty(&StringToNativeStructChain, *NativeStructSubProp);


	
		// 3. Test
		TestTrue(TEXT("Value"), Value == TArray<FName>{ TEXT("Value") });
		TestTrue(TEXT("Vector"), Vector == TArray<FName>{ TEXT("Vector") });
		TestTrue(TEXT("Vector.X"), VectorX == TArray<FName>{ FName(TEXT("Vector")), FName(TEXT("X")) });
		TestTrue(TEXT("NativeStruct"), NativeStruct == TArray<FName>{ FName(TEXT("NativeStruct")) });
		TestTrue(TEXT("NativeStruct.Float"), NativeStructSubProperty == TArray<FName>{ FName(TEXT("NativeStruct")), FName(TEXT("Float")) });
	
		TestTrue(TEXT("FloatArray"), FloatArray == TArray<FName>{ TEXT("FloatArray") });
		TestTrue(TEXT("StringArray"), StringArray == TArray<FName>{ TEXT("StringArray") });
		TestTrue(TEXT("NativeStructArray"), NativeStructArray == TArray<FName>{ TEXT("NativeStructArray") });
		TestTrue(TEXT("NativeStructArray.Float"), NativeStructArraySubProperty == TArray<FName>{ TEXT("NativeStructArray"), TEXT("Float") });
	
		TestTrue(TEXT("FloatSet"), FloatSet == TArray<FName>{ TEXT("FloatSet") });
		TestTrue(TEXT("StringSet"), StringSet == TArray<FName>{ TEXT("StringSet") });
		TestTrue(TEXT("NativeStructSet"), NativeStructSet == TArray<FName>{ TEXT("NativeStructSet") });
		TestTrue(TEXT("NativeStructSet.Float"), NativeStructSetSubProperty == TArray<FName>{ TEXT("NativeStructSet"), TEXT("Float") });
	
		TestTrue(TEXT("StringToFloat"), StringToFloat == TArray<FName>{ TEXT("StringToFloat") });
		TestTrue(TEXT("StringToVector"), StringToVector == TArray<FName>{ TEXT("StringToVector") });
		TestTrue(TEXT("StringToNativeStruct"), StringToNativeStruct == TArray<FName>{ TEXT("StringToNativeStruct") });
		TestTrue(TEXT("StringToNativeStruct.Float"), StringToNativeStructSubProperty == TArray<FName>{ TEXT("StringToNativeStruct"), TEXT("Float") });
	
		return true;
	}

	/**
	 * Tests that all property cases on the specially constructed example class UTestReflectionObject.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertForEachReplictableConcertPropertyTests, "VirtualProduction.Concert.Replication.Data.ForEachReplicatableConcertProperty", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FConcertForEachReplictableConcertPropertyTests::RunTest(const FString& Parameters)
	{
		UClass* Class = UTestReflectionObject::StaticClass();
		TArray<FConcertPropertyChain> Properties;
		UE::ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(*Class, [&Properties](FConcertPropertyChain&& Chain)
		{
			Properties.Add(MoveTemp(Chain));
			return EBreakBehavior::Continue;
		});

		using FExpectedPropertyPath = TArray<FName>;
		const TArray<FExpectedPropertyPath> ExpectedProperties {
			{ TEXT("Float") },
			{ TEXT("Vector") },
			{ TEXT("Vector"), TEXT("X") },
			{ TEXT("Vector"), TEXT("Y") },
			{ TEXT("Vector"), TEXT("Z") },
			{ TEXT("TestStruct") },
			{ TEXT("TestStruct"), TEXT("Nested") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Value") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector"), TEXT("X")  },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector"), TEXT("Y")  },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector"), TEXT("Z")  },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("NativeStruct") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("NativeStruct"), TEXT("Float") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("FloatArray") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringArray") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("NativeStructArray") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("NativeStructArray"), TEXT("Float") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("FloatSet") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringSet") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("NativeStructSet") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("NativeStructSet"), TEXT("Float") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringToFloat") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringToVector") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringToVector"), TEXT("X") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringToVector"), TEXT("Y") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringToVector"), TEXT("Z") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringToNativeStruct") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("StringToNativeStruct"), TEXT("Float") },
			// The same again for NestedArray as for Nested just above 
			{ TEXT("TestStruct"), TEXT("NestedArray") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Value") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Vector") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Vector"), TEXT("X")  },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Vector"), TEXT("Y")  },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Vector"), TEXT("Z")  },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("NativeStruct") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("NativeStruct"), TEXT("Float") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("FloatArray") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringArray") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("NativeStructArray") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("NativeStructArray"), TEXT("Float") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("FloatSet") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringSet") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("NativeStructSet") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("NativeStructSet"), TEXT("Float") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringToFloat") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringToVector") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringToVector"), TEXT("X") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringToVector"), TEXT("Y") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringToVector"), TEXT("Z") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringToNativeStruct") },
			{ TEXT("TestStruct"), TEXT("NestedArray"), TEXT("StringToNativeStruct"), TEXT("Float") },
		};

		for (const FExpectedPropertyPath& Expected : ExpectedProperties)
		{
			// This structure is easier for setting breakpoints than using TestTrue
			if (!Properties.Contains(Expected))
			{
				AddError(
					FString::Printf(
						TEXT("Expected chain %s"),
						*FString::JoinBy(Expected, TEXT("."), [](FName Name){ return Name.ToString(); })
					)
				);
			}
		}
	
		if (Properties.Num() != ExpectedProperties.Num())
		{
			AddError(TEXT("Different number of properties found than expected"));
			for (const FConcertPropertyChain& Chain : Properties)
			{
				if (!ExpectedProperties.Contains(Chain))
				{
					AddError(FString::Printf(TEXT("Found unexpected chain %s"), *Chain.ToString()));
				}
			}
		}
	
		return true;
	}

	/**
	 * Tests FConcertPropertyChain::Matches.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMatchConcertPropertyChainTest, "VirtualProduction.Concert.Replication.Data.MatchConcertPropertyChain", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMatchConcertPropertyChainTest::RunTest(const FString& Parameters)
	{
		// 1. Set up
		UClass* Class = UTestReflectionObject::StaticClass();
		FProperty* FloatProperty = Class->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestReflectionObject, Float));
		FProperty* TestStructProperty = Class->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestReflectionObject, TestStruct));
		FProperty* NestedArrayProperty = FTestNestedReplicationStruct::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestNestedReplicationStruct, NestedArray));
		FProperty* InternalNestedArrayProperty = CastField<FArrayProperty>(NestedArrayProperty)->Inner;
		FProperty* FloatArrayProperty = FTestReplicationStruct::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, FloatArray));
		FProperty* StringToFloatProperty = FTestReplicationStruct::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FTestReplicationStruct, StringToFloat));
	
		FArchiveSerializedPropertyChain ChainToNestedArrayProperty;
		ChainToNestedArrayProperty.PushProperty(TestStructProperty, false);
		// FTestReplicationStruct::Value (float)
		FArchiveSerializedPropertyChain ChainToNestedProperty = ChainToNestedArrayProperty;
		// FTestReplicationStruct::FloatArray (TArray<float>)
		ChainToNestedProperty.PushProperty(NestedArrayProperty, false);
		ChainToNestedProperty.PushProperty(InternalNestedArrayProperty, false);
		FArchiveSerializedPropertyChain ChainToFloatArrayProperty = ChainToNestedProperty;
		FArchiveSerializedPropertyChain ChainToInternalFloatArrayProperty = ChainToFloatArrayProperty;
		ChainToInternalFloatArrayProperty.PushProperty(FloatArrayProperty, false);
		// FTestReplicationStruct::StringToFloat (TMap<FString, float>)
		FArchiveSerializedPropertyChain ChainToStringToFloatProperty = ChainToNestedProperty;
		FArchiveSerializedPropertyChain ChainToStringToFloatValueProperty = ChainToNestedProperty;
		ChainToStringToFloatValueProperty.PushProperty(StringToFloatProperty, false);
	
		const FConcertPropertyChain ConcertChain_Simple_FloatProperty(nullptr, *FloatProperty);
		const FConcertPropertyChain ConcertChain_Nested_ArrayProperty(&ChainToNestedArrayProperty, *NestedArrayProperty);
		const FConcertPropertyChain ConcertChain_Nested_FloatArrayProperty(&ChainToFloatArrayProperty, *FloatArrayProperty);
		const FConcertPropertyChain ConcertChain_Nested_StringToFloatProperty(&ChainToStringToFloatProperty, *StringToFloatProperty);
	


		// 2. Run
		const bool bMatches_Simple_FloatProperty = ConcertChain_Simple_FloatProperty.MatchesExactly(nullptr, *FloatProperty);
		const bool bMatches_Nested_ArrayProperty = ConcertChain_Nested_ArrayProperty.MatchesExactly(&ChainToNestedArrayProperty, *NestedArrayProperty);
		const bool bMatches_Nested_FloatArrayProperty = ConcertChain_Nested_FloatArrayProperty.MatchesExactly(&ChainToFloatArrayProperty, *FloatArrayProperty);
		const bool bMatches_Nested_StringToFloatProperty = ConcertChain_Nested_StringToFloatProperty.MatchesExactly(&ChainToStringToFloatProperty, *StringToFloatProperty);




		// 3. Test
		TestTrue(TEXT("Float"), bMatches_Simple_FloatProperty);
		TestTrue(TEXT("NestedArray"), bMatches_Nested_ArrayProperty);
		TestTrue(TEXT("NestedArray.FloatArray"), bMatches_Nested_FloatArrayProperty);
		TestTrue(TEXT("NestedArray.StringToFloat"), bMatches_Nested_StringToFloatProperty);

		return true;
	}

	/**
	 * Tests that UE::ConcertSyncCore::FReplicationPropertyFilter works correctly.
	 * This uses the filter and serializes an object, doing a Memcmp to detect that exactly the expected properties are serialized.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSerializeWithPropertyFilterTest, "VirtualProduction.Concert.Replication.Data.SerializeWithPropertyFilter", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FSerializeWithPropertyFilterTest::RunTest(const FString& Parameters)
	{
		class FTestArchive : public FMemoryWriter
		{
			FAutomationTestBase& Test;
			const FConcertPropertySelection& Selection;
			const UE::ConcertSyncCore::FReplicationPropertyFilter PropertyFilter;
		public:
		
			TArray<uint8> Bytes;

			FTestArchive(FAutomationTestBase& Test, const FConcertPropertySelection& Selection)
				: FMemoryWriter(Bytes)
				, Test(Test)
				, Selection(Selection)
				, PropertyFilter(Selection)
			{
				FArchive::SetIsSaving(true);
			}

			virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
			{
				if (!UE::ConcertSyncCore::PropertyChain::IsReplicatableProperty(*InProperty))
				{
					return true;
				}

				// We can assume the constructor of FConcertPropertyChain works correctly here since we've tested it in another test.
				const FConcertPropertyChain Chain(GetSerializedPropertyChain(), *InProperty);
				const bool bIsPropertyInChain = Selection.ReplicatedProperties.Contains(Chain);
				const bool bIsAllowed = PropertyFilter.ShouldSerializeProperty(GetSerializedPropertyChain(), *InProperty);
				if (bIsPropertyInChain)
				{
					Test.TestTrue(FString::Printf(TEXT("Allowed %s"), *InProperty->GetAuthoredName()), bIsAllowed);
				}
				else
				{
					Test.TestFalse(FString::Printf(TEXT("Disallowed %s"), *InProperty->GetAuthoredName()), bIsAllowed);
				}

				return !bIsAllowed;
			}

			// If we encounter any objects, ring alarm bells because IsReplicatableProperty should have covered this case
			virtual FArchive& operator<<(UObject*& Obj) override{ Test.AddError(TEXT("Object not expected at this point")); return *this; }
			virtual FArchive& operator<<(FWeakObjectPtr& Obj) override { Test.AddError(TEXT("Object not expected at this point")); return *this; }
			virtual FArchive& operator<<(FSoftObjectPtr& Value) override { Test.AddError(TEXT("Object not expected at this point")); return *this; } 
			virtual FArchive& operator<<(FSoftObjectPath& Value) override { Test.AddError(TEXT("Object not expected at this point")); return *this; }
			virtual FArchive& operator<<(FObjectPtr& Obj) override { Test.AddError(TEXT("Object not expected at this point")); return *this; }
		};

		// CreateFromPath is supposed to work (assuming the other tests worked so far) but we'll use TOptional::Get() calls are in case it does not. 
		TSet<FConcertPropertyChain> AllowedProperties = {
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Float") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector"), TEXT("X") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector"), TEXT("Y") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector"), TEXT("Z") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("Nested") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("Nested"), TEXT("Value") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("Nested"), TEXT("NativeStruct") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("Nested"), TEXT("NativeStruct") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("Nested"), TEXT("FloatArray") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Value") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Vector") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Vector"), TEXT("X") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Vector"), TEXT("Y") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Vector"), TEXT("Z") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray"), TEXT("NativeStruct") }).Get({}),
			FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray"), TEXT("FloatArray") }).Get({}),
		};
		if (AllowedProperties.Contains(FConcertPropertyChain{}))
		{
			AddError(TEXT("FConcertPropertyChain::CreateFromPath was expected to work on all of the above cases. Take a look at the ForEachReplicatableConcertProperty test, which should have failed, as well."));
			return false;
		}

		auto SetTestValues = [](UTestReflectionObject& Object)
		{
			Object.Float = 42.f;
			Object.Vector = { 10.f, 20.f, 30.f };
		
			Object.TestStruct.Nested.Value = 420.f;
			Object.TestStruct.Nested.NativeStruct.Float = 4200.f;
			Object.TestStruct.Nested.FloatArray = { 500.f, 750.f };

			Object.TestStruct.NestedArray = { FTestReplicationStruct{} };
			Object.TestStruct.NestedArray[0].Value = -420.f;
			Object.TestStruct.NestedArray[0].Vector = { -100.f, -200.f, -300.f };
			Object.TestStruct.NestedArray[0].NativeStruct.Float = -4200.f;
			Object.TestStruct.NestedArray[0].FloatArray = { -500.f, -750.f };
		};
		auto ResetTestValues = [](UTestReflectionObject& Object)
		{
			Object.Float = {};
			Object.Vector = FVector::ZeroVector;
			Object.TestStruct = FTestNestedReplicationStruct{};
		};
		auto ObjectToBinary = [](UObject& Object)
		{
			TArray<uint8> Bytes;
			FObjectWriter CollectAllData(&Object, Bytes, false);
			return Bytes;
		};
		auto ValidateTestValues = [this](UTestReflectionObject& Object)
		{
			TestEqual(TEXT("Object.Float = 42.f"), Object.Float, 42.f);
			TestEqual(TEXT("Object.Vector = { 10.f, 20.f, 30.f }"), Object.Vector, FVector{10.f, 20.f, 30.f });
		
			TestEqual(TEXT("Object.TestStruct.Nested.Value = 420.f"), Object.TestStruct.Nested.Value, 420.f);
			TestEqual(TEXT("Object.TestStruct.Nested.NativeStruct.Float = 4200.f"), Object.TestStruct.Nested.NativeStruct.Float, 4200.f);
			TestEqual(TEXT("Object.TestStruct.Nested.FloatArray = { 500.f, 750.f }"), Object.TestStruct.Nested.FloatArray, TArray<float>{ 500.f, 750.f });

			TestEqual(TEXT("Object.TestStruct.NestedArray[0].Value = -420.f"), Object.TestStruct.NestedArray[0].Value, -420.f);
			TestEqual(TEXT("Object.TestStruct.NestedArray[0].Vector = { -100.f, -200.f, -300.f }"), Object.TestStruct.NestedArray[0].Vector, FVector{ -100.f, -200.f, -300.f });
			TestEqual(TEXT("Object.TestStruct.NestedArray[0].NativeStruct.Float = -4200.f"), Object.TestStruct.NestedArray[0].NativeStruct.Float, -4200.f);
			TestEqual(TEXT("Object.TestStruct.NestedArray[0].FloatArray = { -500.f, -750.f }"), Object.TestStruct.NestedArray[0].FloatArray, TArray<float>{ -500.f, -750.f });
		};
	
		UTestReflectionObject* TestObject = NewObject<UTestReflectionObject>();
		SetTestValues(*TestObject);
	
		// This property is supposed to be filtered out...
		TestObject->TestStruct.Nested.Vector = { 100.f, 200.f, 300.f };
		const TArray<uint8> SnapshotWithTestValues = ObjectToBinary(*TestObject);
		// ... and we change its value after taking snapshotting to validate the filter indeed filters out the undesired property...
		TestObject->TestStruct.Nested.Vector = { 1000.f, 2000.f, 3000.f };

		// This does the filtering & tests that properties are filtered correctly using ShouldSkipProperty
		const FConcertPropertySelection Selection{ AllowedProperties };
		FTestArchive SavingArchive(*this, Selection);
		FNameAsStringProxyArchive CrashProtection_Saving(SavingArchive);
		TestObject->Serialize(CrashProtection_Saving);

		// ... the supposed to be filtered property must now be reset to the same value as was in the snapshot so Memcmp does not detect it.
		ResetTestValues(*TestObject);
		TestObject->TestStruct.Nested.Vector = { 100.f, 200.f, 300.f };
		FObjectReader LoadingArchive(SavingArchive.Bytes);
		FNameAsStringProxyArchive CrashProtection_Loading(LoadingArchive);
		TestObject->Serialize(CrashProtection_Loading);
		const TArray<uint8> SnapshotAfterLoading = ObjectToBinary(*TestObject);
	
		// We do a Memcmp on the snapshot data before filtering and on the data after applying to a blanked out object. They must be equal or the filter did not work correctly!
		if (SnapshotWithTestValues.Num() == SnapshotAfterLoading.Num())
		{
			const bool bEqualMemory = FPlatformMemory::Memcmp(SnapshotWithTestValues.GetData(), SnapshotAfterLoading.GetData(), SnapshotWithTestValues.Num()) == 0;
			TestTrue(TEXT("Memcmp returned non-zero: Applying the filtered data to a reset object did not leave the object in the same state as it was serialized in."), bEqualMemory);
		}
		else
		{
			AddError(TEXT("BinaryDataWithTestValues != BinaryDataAfterLoading: Applying the filtered data to a reset object did not leave the object in the same state as it was serialized in."));
		}

		// This is technically unnecessary because Memcmp would already have failed the test above but this helps in debugging what values caused the Memcmp to be different.
		ValidateTestValues(*TestObject);
		TestEqual(TEXT("Object.TestStruct.Nested.Vector was left unchanged"), TestObject->TestStruct.Nested.Vector, FVector{ 100.f, 200.f, 300.f });
		return true;
	}

	/**
	 * Tests that UE::ConcertSyncCore::FReplicationPropertyFilter works correctly.
	 * This uses the filter and serializes an object, doing a Memcmp to detect that exactly the expected properties are serialized.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOverlapPropertiesTest, "VirtualProduction.Concert.Replication.Data.OverlapProperties", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FOverlapPropertiesTest::RunTest(const FString& Parameters)
	{
		// 1. Define paths
		using FExpectedPropertyPath = TArray<FName>;
		const TArray<FExpectedPropertyPath> RootProperties {
			{ TEXT("Float") },
			{ TEXT("Vector") },
			{ TEXT("Vector"), TEXT("X") },
			{ TEXT("Vector"), TEXT("Y") },
			{ TEXT("Vector"), TEXT("Z") },
			};
		const TArray<FExpectedPropertyPath> NestedProperties = {
			{ TEXT("TestStruct") },
			{ TEXT("TestStruct"), TEXT("Nested") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Value") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector") },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector"), TEXT("X")  },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector"), TEXT("Y")  },
			{ TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector"), TEXT("Z")  }
			};
		const TArray<FExpectedPropertyPath> Overlap = {
			{ TEXT("Vector"), TEXT("X") },
			{ TEXT("TestStruct") },
		};

		// 2. Convert to FConcertPropertyChain
		FConcertPropertySelection RootPropertySelection;
		FConcertPropertySelection NestedPropertySelection;
		FConcertPropertySelection OverlapSelection;
		auto TransformOp = [this](const FExpectedPropertyPath& Path)
		{
			const TOptional<FConcertPropertyChain> Result = FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), Path);
			if (!Result)
			{
				AddError(TEXT("Failed to convert all paths"));
			}
			return Result.Get({});
		};
		Algo::Transform(RootProperties, RootPropertySelection.ReplicatedProperties, TransformOp);
		Algo::Transform(NestedProperties, NestedPropertySelection.ReplicatedProperties, TransformOp);
		Algo::Transform(Overlap, OverlapSelection.ReplicatedProperties, TransformOp);


		// 3. Test permutations
		TestFalse(TEXT("Root <> Nested"), RootPropertySelection.OverlapsWith(NestedPropertySelection));
		TestFalse(TEXT("Nested <> Root"), NestedPropertySelection.OverlapsWith(RootPropertySelection));
		
		TestTrue(TEXT("Root <> Overlap"), RootPropertySelection.OverlapsWith(OverlapSelection));
		TestTrue(TEXT("Nested <> Overlap"), NestedPropertySelection.OverlapsWith(OverlapSelection));
		TestTrue(TEXT("Overlap <> Root"), OverlapSelection.OverlapsWith(RootPropertySelection));
		TestTrue(TEXT("Overlap <> Nested"), OverlapSelection.OverlapsWith(NestedPropertySelection));
		
		return true;
	}
}