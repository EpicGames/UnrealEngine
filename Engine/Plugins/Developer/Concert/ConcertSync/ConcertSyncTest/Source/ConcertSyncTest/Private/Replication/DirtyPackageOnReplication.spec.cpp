// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Util/ConcertClientReplicationBridgeMock.h"
#include "Util/Spec/ObjectTestReplicator.h"
#include "Util/Spec/ReplicationClient.h"
#include "Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	BEGIN_DEFINE_SPEC(FDirtyPackageOnReplication, "VirtualProduction.Concert.Replication.DirtyPackageOnReplication", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		UPackage* Package;
		TUniquePtr<FObjectTestReplicator> Replicator;
		TUniquePtr<FReplicationServer> Server;
	
		FReplicationClient* Sender;
		FReplicationClient* Receiver;

		const FGuid StreamId = FGuid::NewGuid();
	END_DEFINE_SPEC(FDirtyPackageOnReplication);

	/** In the build farm, MarkPackageDirty can fail for yet unknown reasons. This just skips the rest if MarkPackageDirty does not work. */
#define SKIP_IF_CANNOT_DIRTY() \
	Replicator->TestObject->MarkPackageDirty(); \
	if (!Package->IsDirty()) \
	{ \
	return; \
	} \
	
	/**
	 * This tests that packages are dirtied whenever anything is replicated in them.
	 * This is important so Multi-User reverts the package back to its original state when leaving replication.
	 */
	void FDirtyPackageOnReplication::Define()
	{
		BeforeEach([this]
		{
			Package = NewObject<UPackage>(nullptr, *FString::Printf(TEXT("/Engine/Transient/%s"), *FGuid::NewGuid().ToString()), RF_Transient);
			Replicator = MakeUnique<FObjectTestReplicator>(Package);
			Server = MakeUnique<FReplicationServer>(*this);
			Sender = &Server->ConnectClient();
			Receiver = &Server->ConnectClient();

			Sender->JoinReplication(Replicator->CreateSenderArgs(StreamId));
			Receiver->JoinReplicationAsListener({ Replicator->TestObject });
			Sender->GetBridgeMock().InjectAvailableObject(*Replicator->TestObject);
			Sender->GetClientReplicationManager().TakeAuthorityOver({ Replicator->TestObject });

			Package->ClearDirtyFlag();
			TestTrue(TEXT("Test set-up correctly"), !Package->IsDirty());
		});
		AfterEach([this]
		{
			Server.Reset();
		});

		It("Package is marked dirty when replicated", [this]
		{
			SKIP_IF_CANNOT_DIRTY();
			
			Replicator->SimulateSendObjectToReceiver(*this, { *Sender, *Server, *Receiver }, { StreamId });
			Replicator->TestValuesWereReplicated(*this);

			TestTrue(TEXT("IsDirty()"), Package->IsDirty());
		});

		It("Package is re-marked dirty when saved in between replication", [this]
		{
			SKIP_IF_CANNOT_DIRTY();
			
			Replicator->SimulateSendObjectToReceiver(*this, { *Sender, *Server, *Receiver }, { StreamId });
			Replicator->TestValuesWereReplicated(*this);

			// Simulate a save
			Package->ClearDirtyFlag();

			Replicator->SimulateSendObjectToReceiver(*this, { *Sender, *Server, *Receiver }, { StreamId });
			Replicator->TestValuesWereReplicated(*this);
			
			TestTrue(TEXT("IsDirty()"), Package->IsDirty());
		});
	}
#undef SKIP_IF_CANNOT_DIRTY
}
