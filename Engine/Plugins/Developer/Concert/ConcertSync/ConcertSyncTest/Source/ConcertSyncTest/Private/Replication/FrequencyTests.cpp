// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Data/ReplicationStream.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Util/SendReceiveObjectTestBase.h"
#include "TestReflectionObject.h"

#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"

#if WITH_AUTOMATION_TESTS

namespace UE::ConcertSyncTests::Replication::Frequency
{
	/** Makes the test object replicate at a frequency of 30 times per second. */
	class FSendReceiveObjectWithFrequencyTest : public FSendReceiveObjectTestBase
	{
	public:
		FSendReceiveObjectWithFrequencyTest(const FString& InName, const bool bInComplexTask)
			: FSendReceiveObjectTestBase(InName, bInComplexTask)
		{}

		static constexpr uint8 GetUpdateRate() { return 30; } 
		static constexpr double GetUpdateInterval() { return 1.0 / static_cast<double>(GetUpdateRate()); } 

		//~ Begin FSendReceiveTestBase Interface
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateSenderArgs() override
		{
			// CreateHandshakeArgsFrom defaults to EConcertObjectReplicationMode::Realtime which must be overriden for this test.
			constexpr EConcertObjectReplicationMode ReplicationMode = EConcertObjectReplicationMode::SpecifiedRate;
			return CreateHandshakeArgsFrom(*TestObject, SenderStreamId, ReplicationMode, GetUpdateRate());
		}
		//~ End FSendReceiveTestBase Interface
	};
	
	DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER(FSendAndCountReceivedCommand, FString, Name, FSendReceiveObjectTestBase*, Test, uint32, ExpectedNumServerToReceive, uint32, ExpectedNumClientToReceive);
	bool FSendAndCountReceivedCommand::Update()
	{
		uint32 NumServerReceived = 0;
		uint32 NumClientReceived = 0;
		auto OnServerReceive = [&NumServerReceived](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
		{
			++NumServerReceived;
		};
		auto OnClientReceive = [&NumClientReceived](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
		{
			++NumClientReceived;
		};
		Test->SimulateSendObjectToReceiver(OnServerReceive, OnClientReceive);
		
		Test->TestEqual(FString::Printf(TEXT("%s: Server received data"), *Name), NumServerReceived, ExpectedNumServerToReceive);
		Test->TestEqual(FString::Printf(TEXT("%s: Client received data"), *Name), NumClientReceived, ExpectedNumClientToReceive);
		return true;
	}
	
	/** Tests that something is sent after the replication interval has expired. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FReceiveAfterSpecifiedFrequencyInterval, FSendReceiveObjectWithFrequencyTest, "VirtualProduction.Concert.Replication.Frequency.SendingClientRespectsFrequency", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FReceiveAfterSpecifiedFrequencyInterval::RunTest(const FString& Parameters)
	{
		SetUpClientAndServer();
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObject });

		// 1st frame replicates ...
		ADD_LATENT_AUTOMATION_COMMAND(FSendAndCountReceivedCommand(TEXT("Frame 1 sends"), this, 1, 1));
		// ... subsequent frames must wait
		ADD_LATENT_AUTOMATION_COMMAND(FSendAndCountReceivedCommand(TEXT("Frame 2 does not send"), this, 0, 0));
		ADD_LATENT_AUTOMATION_COMMAND(FSendAndCountReceivedCommand(TEXT("Frame 3 does not send"), this, 0, 0));
		ADD_LATENT_AUTOMATION_COMMAND(FSendAndCountReceivedCommand(TEXT("Frame 4 does not send"), this, 0, 0));
		ADD_LATENT_AUTOMATION_COMMAND(FSendAndCountReceivedCommand(TEXT("Frame 5 does not send"), this, 0, 0));

		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(GetUpdateInterval()));
		// ... until GetUpdateInterval() has elapsed
		ADD_LATENT_AUTOMATION_COMMAND(FSendAndCountReceivedCommand(TEXT("Later frame sends again"), this, 1, 1));
		return true;
	}

	// TODO UE-203616: This test sometimes fails. Find out why.
	/** Tests that after about 1 second, the expected number of events was received. */
	/*IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FSimulateAt30FPS, FSendReceiveObjectWithFrequencyTest, "VirtualProduction.Concert.Replication.Frequency.SimulateAt30FPS", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FSimulateAt30FPS::RunTest(const FString& Parameters)
	{
		// 1. Set up server
		SetUpClientAndServer();
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObject });

		// 2. Simulate the engine ticking for 1 second
		// This was tried with ADD_LATENT_AUTOMATION_COMMAND but the test results were too inconsistent and imprecise
		uint32 NumServerReceived = 0;
		uint32 NumClientReceived = 0;
		bool bIsFirstServerReceive = true;
		bool bIsFirstClientReceive = true;
		
		double CurrentSeconds = FPlatformTime::Seconds();
		double LastServerReceiveTime = CurrentSeconds;
		double LastClientReceiveTime = CurrentSeconds;
		
		constexpr double UpdateIntervalSeconds = GetUpdateInterval();
		const double EndSeconds = CurrentSeconds + 1.0;
		for (; CurrentSeconds < EndSeconds; CurrentSeconds = FPlatformTime::Seconds())
		{
			const double DeltaSecSinceLast_Server = CurrentSeconds - LastServerReceiveTime;
			const double DeltaSecSinceLast_Client = CurrentSeconds - LastClientReceiveTime;
			
			bool bReceivedServer = false;
			bool bReceivedClient = false;
			auto OnServerReceive = [&](const FConcertSessionContext&, const FConcertReplication_BatchReplicationEvent&)
			{
				LastServerReceiveTime = CurrentSeconds;
				++NumServerReceived;
				bReceivedServer = true;
			};
			auto OnClientReceive = [&](const FConcertSessionContext&, const FConcertReplication_BatchReplicationEvent&)
			{
				LastClientReceiveTime = CurrentSeconds;
				++NumClientReceived;
				bReceivedClient = true;
			};
			// This simulates an engine tick
			SimulateSendObjectToReceiver(OnServerReceive, OnClientReceive);

			// 2.1 Verify that every received update is in acceptable percentage range
			constexpr double AcceptableTimeRange = UpdateIntervalSeconds * 0.01;
			constexpr double MinDeltaTime = UpdateIntervalSeconds - AcceptableTimeRange;
			constexpr double MaxDeltaTime = UpdateIntervalSeconds + AcceptableTimeRange;
			if (bReceivedServer)
			{
				const bool bMin = MinDeltaTime <= DeltaSecSinceLast_Server || bIsFirstServerReceive;
				const bool bMax = DeltaSecSinceLast_Server <= MaxDeltaTime || bIsFirstServerReceive;
				AddErrorIfFalse(bMin, FString::Printf(TEXT("Server: MinDeltaTime (%f) <= DeltaSecSinceLast_Server (%f)"), MinDeltaTime, DeltaSecSinceLast_Server));
				AddErrorIfFalse(bMax, FString::Printf(TEXT("Server: DeltaSecSinceLast_Server (%f) <= MaxDeltaTime (%f)"), DeltaSecSinceLast_Server, MaxDeltaTime));
				bIsFirstServerReceive = false;
			}
			if (bReceivedClient)
			{
				const bool bMin = MinDeltaTime <= DeltaSecSinceLast_Client || bIsFirstClientReceive;
				const bool bMax = DeltaSecSinceLast_Client <= MaxDeltaTime || bIsFirstClientReceive;
				AddErrorIfFalse(bMin, FString::Printf(TEXT("Client: MinDeltaTime (%f) <= DeltaSecSinceLast_Client (%f)"), MinDeltaTime, DeltaSecSinceLast_Client));
				AddErrorIfFalse(bMax, FString::Printf(TEXT("Client: DeltaSecSinceLast_Client (%f) <= MaxDeltaTime (%f)"), DeltaSecSinceLast_Client, MaxDeltaTime));
				bIsFirstClientReceive = false;
			}
		}

		// 2.2 Verify that the expected number of events was received in the past 1 second
		// We may be off by one event due to double rounding errors
		constexpr uint8 AcceptableDropAmount = 1;
		constexpr uint8 MinExpected = GetUpdateRate() - AcceptableDropAmount;
		constexpr uint8 MaxExpected = GetUpdateRate();
		AddErrorIfFalse(MinExpected <= NumServerReceived, FString::Printf(TEXT("Server: MinExpected (%d)<= NumServerReceived (%d)"), MinExpected, NumServerReceived));
		AddErrorIfFalse(NumServerReceived <= MaxExpected, FString::Printf(TEXT("Server: NumServerReceived (%d) <= MaxExpected (%d)"), NumServerReceived, MaxExpected));
		AddErrorIfFalse(MinExpected <= NumClientReceived, FString::Printf(TEXT("Server: MinExpected (%d)<= NumClientReceived (%d)"), MinExpected, NumClientReceived));
		AddErrorIfFalse(NumClientReceived <= MaxExpected, FString::Printf(TEXT("Server: NumClientReceived (%d) <= MaxExpected (%d)"), NumClientReceived, MaxExpected));

		// 2.1 (time between updates) and 2.2 (number of updates) correctly test that we replicate at the expected frequency.
		return true;
	}*/
}

#endif