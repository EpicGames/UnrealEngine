// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication
{
	/** This tests that muting & unmuting retains a consistent check across various types of changes. */
	BEGIN_DEFINE_SPEC(FMuteAndQuerySpec, "VirtualProduction.Concert.Replication.Muting", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	TSharedPtr<FReplicationServer> Server;
	FReplicationClient* Sender = nullptr;

	FGuid StreamId_Bar					= FGuid::NewGuid();
	FGuid StreamId_BarSubobject			= FGuid::NewGuid();
	FGuid StreamId_BarNestedSubobject	= FGuid::NewGuid();
	// Leverage FObjectTestReplicator to create more UObjects
	TSharedPtr<FObjectTestReplicator> Replicator_Bar;			
	TSharedPtr<FObjectTestReplicator> Replicator_BarSubobject;		
	TSharedPtr<FObjectTestReplicator> Replicator_BarNestedSubobject;
	END_DEFINE_SPEC(FMuteAndQuerySpec);

	/** This tests that muting requests work when EConcertSyncSessionFlags::ShouldAllowGlobalMuting is set. */
	void FMuteAndQuerySpec::Define()
	{
		BeforeEach([this]
		{
			Server		= MakeShared<FReplicationServer>(*this);
			Sender		= &Server->ConnectClient();

			Replicator_Bar					= MakeShared<FObjectTestReplicator>();
			Replicator_BarSubobject			= Replicator_Bar->CreateSubobjectReplicator();
			Replicator_BarNestedSubobject	= Replicator_BarSubobject->CreateSubobjectReplicator();
		});
		AfterEach([this]
		{
			// Test would hold onto this for rest of engine lifetime. Clean up this mini would-be leak.
			Server.Reset();
			Replicator_Bar.Reset();
			Replicator_BarSubobject.Reset();
			Replicator_BarNestedSubobject.Reset();
		});

		// These tests involve only using mute & unmute requests (for modifying stream contents see further below)
		Describe("Muting hierarchy (only mute & unmute requests)", [this]()
		{
			BeforeEach([this]()
			{
				ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderJoinArgs;
				SenderJoinArgs.Streams.Add(Replicator_Bar->CreateStream(StreamId_Bar));
				SenderJoinArgs.Streams.Add(Replicator_BarSubobject->CreateStream(StreamId_BarSubobject));
				SenderJoinArgs.Streams.Add(Replicator_BarNestedSubobject->CreateStream(StreamId_BarNestedSubobject));
				
				Sender->JoinReplication(SenderJoinArgs);
				Sender->GetClientReplicationManager().MuteObjects({ Replicator_Bar->TestObject }, EConcertReplicationMuteOption::ObjectAndSubobjects);
			});
			
			It("Query entire hierarchy", [this]()
			{
				// Requesting all muted objects
				bool bReceivedQueryAllEvent = false;
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this, &bReceivedQueryAllEvent](FConcertReplication_QueryMuteState_Response&& Response)
					{
						bReceivedQueryAllEvent = true;
						TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 1"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 0"), Response.ExplicitlyUnmutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 2"), Response.ImplicitlyMutedObjects.Num(), 2);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 0"), Response.ImplicitlyUnmutedObjects.Num(), 0);
						
						const FConcertReplication_ObjectMuteSetting* ObjectOneMuteSetting = Response.ExplicitlyMutedObjects.Find(Replicator_Bar->TestObject);
						TestTrue(TEXT("ExplicitlyMutedObjects.Contains(Bar)"), ObjectOneMuteSetting && ObjectOneMuteSetting->Flags == EConcertReplicationMuteOption::ObjectAndSubobjects);
						TestTrue(TEXT("ImplicitlyMutedObjects.Contains(BarSubobject)"), Response.ImplicitlyMutedObjects.Contains(Replicator_BarSubobject->TestObject));
						TestTrue(TEXT("ImplicitlyMutedObjects.Contains(BarNestedSubobject)"), Response.ImplicitlyMutedObjects.Contains(Replicator_BarNestedSubobject->TestObject));
						
					});
				TestTrue(TEXT("Received query all event"), bReceivedQueryAllEvent);
			});
			
			It("Query only BarSubobject", [this]()
			{
				Sender->GetClientReplicationManager()
					.QueryMuteState({ Replicator_BarSubobject->TestObject })
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 0"), Response.ExplicitlyMutedObjects.Num(), 0);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 0"), Response.ExplicitlyUnmutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 1"), Response.ImplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 0"), Response.ImplicitlyUnmutedObjects.Num(), 0);
						
						TestTrue(TEXT("ImplicitlyMutedObjects.Contains(BarSubobject)"), Response.ImplicitlyMutedObjects.Contains(Replicator_BarSubobject->TestObject));
					});
			});
			
			It("Unmute subobject", [this]()
			{
				Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_BarSubobject->TestObject }, EConcertReplicationMuteOption::OnlyObject);
				
				bool bReceivedEvent = false;
				Sender->GetClientReplicationManager()
					.QueryMuteState({ Replicator_Bar->TestObject, Replicator_BarSubobject->TestObject })
					.Next([this, &bReceivedEvent](FConcertReplication_QueryMuteState_Response&& Response)
					{
						bReceivedEvent = true;
						
						TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 1"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 1"), Response.ExplicitlyUnmutedObjects.Num(), 1);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 0"), Response.ImplicitlyMutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 0"), Response.ImplicitlyUnmutedObjects.Num(), 0);
					
						const FConcertReplication_ObjectMuteSetting* ObjectOneMuteSetting = Response.ExplicitlyMutedObjects.Find(Replicator_Bar->TestObject);
						TestTrue(TEXT("Root object still explicitly muted"), ObjectOneMuteSetting && ObjectOneMuteSetting->Flags == EConcertReplicationMuteOption::ObjectAndSubobjects);
					
						const FConcertReplication_ObjectMuteSetting* ObjectTwoMuteSetting = Response.ExplicitlyUnmutedObjects.Find(Replicator_BarSubobject->TestObject);
						TestTrue(TEXT("Subobject explicitly unmuted"), ObjectTwoMuteSetting && ObjectTwoMuteSetting->Flags == EConcertReplicationMuteOption::OnlyObject);
					});
				TestTrue(TEXT("Received query response"), bReceivedEvent);
			});

			It("Unmuting hiearchy by unmuting root object ", [this]()
			{
				Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_Bar->TestObject });
				
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestTrue(TEXT("IsEmpty()"), Response.IsEmpty());
					});
			});
			
			It("Unmute hierarchy by changing root to EConcertReplicationMuteFlags::None", [this]()
			{
				Sender->GetClientReplicationManager().MuteObjects({ Replicator_Bar->TestObject }, EConcertReplicationMuteOption::OnlyObject);
				
				Sender->GetClientReplicationManager()
					.QueryMuteState({ Replicator_BarSubobject->TestObject })
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestTrue(TEXT("IsEmpty"), Response.IsEmpty());
					});
			});

			It("Double mute hierarchy", [this]()
			{
				// The point of this test is that in this case
				// - Bar <- is IncludeSubobjects
				//	- BarSubobject <- change from IncludeSubobjects to None
				//		- BarNestedSubobject <- stays implicitly muted
				// the system detects that NestedSubobject should remain muted due to Root.
				Sender->GetClientReplicationManager().MuteObjects({ Replicator_BarSubobject->TestObject }, EConcertReplicationMuteOption::ObjectAndSubobjects);
				Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_BarSubobject->TestObject }, EConcertReplicationMuteOption::OnlyObject);
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 1"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 1"), Response.ExplicitlyUnmutedObjects.Num(), 1);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 1"), Response.ImplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 0"), Response.ImplicitlyUnmutedObjects.Num(), 0);
					
						const FConcertReplication_ObjectMuteSetting* BarMuteSetting = Response.ExplicitlyMutedObjects.Find(Replicator_Bar->TestObject);
						TestTrue(TEXT("ExplicitlyMutedObjects.Contains(Bar)"), BarMuteSetting && BarMuteSetting->Flags == EConcertReplicationMuteOption::ObjectAndSubobjects);
						const FConcertReplication_ObjectMuteSetting* SubobjectMuteSetting = Response.ExplicitlyUnmutedObjects.Find(Replicator_BarSubobject->TestObject);
						TestTrue(TEXT("ExplicitlyUnmutedObjects.Contains(BarSubobject)"), SubobjectMuteSetting && SubobjectMuteSetting->Flags == EConcertReplicationMuteOption::OnlyObject);
						TestTrue(TEXT("ImplicitlyMutedObjects.Contains(BarNestedSubobject)"), Response.ImplicitlyMutedObjects.Contains(Replicator_BarNestedSubobject->TestObject));
					});
			});
		});

		It("Changes are atomic", [this]()
		{
			Sender->JoinReplication(Replicator_Bar->CreateSenderArgs(StreamId_Bar));
			
			// This is supposed to fail because Replicator_BarSubobject is not registered
			Sender->GetClientReplicationManager().MuteObjects({ Replicator_Bar->TestObject, Replicator_BarSubobject->TestObject });

			Sender->GetClientReplicationManager()
				.QueryMuteState()
				.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
				{
					TestTrue(TEXT("IsEmpty()"), Response.IsEmpty());
				});
		});

		It("Implicitly unmute nested subobject", [this]()
		{
			ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderJoinArgs;
			SenderJoinArgs.Streams.Add(Replicator_Bar->CreateStream(StreamId_Bar));
			SenderJoinArgs.Streams.Add(Replicator_BarSubobject->CreateStream(StreamId_BarSubobject));
			SenderJoinArgs.Streams.Add(Replicator_BarNestedSubobject->CreateStream(StreamId_BarNestedSubobject));
			Sender->JoinReplication(SenderJoinArgs);
			
			FConcertReplication_ChangeMuteState_Request MuteRequest;
			MuteRequest.ObjectsToMute.Add(Replicator_Bar->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects });
			MuteRequest.ObjectsToUnmute.Add(Replicator_BarSubobject->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects });
			Sender->GetClientReplicationManager().ChangeMuteState(MuteRequest);
			
			Sender->GetClientReplicationManager()
				.QueryMuteState()
				.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
				{
					TestTrue(TEXT("IsSuccess"), Response.IsSuccess());
					
					TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 1"), Response.ExplicitlyMutedObjects.Num(), 1);
					TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 1"), Response.ExplicitlyUnmutedObjects.Num(), 1);
					TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 1"), Response.ImplicitlyMutedObjects.Num(), 0);
					TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 1"), Response.ImplicitlyUnmutedObjects.Num(), 1);

					const FConcertReplication_ObjectMuteSetting* ExplicitlyMuted = Response.ExplicitlyMutedObjects.Find(Replicator_Bar->TestObject);
					const FConcertReplication_ObjectMuteSetting* ExplicitlyUnmuted = Response.ExplicitlyUnmutedObjects.Find(Replicator_BarSubobject->TestObject);
					
					TestTrue(TEXT("Root is muted"), ExplicitlyMuted != nullptr);
					TestTrue(TEXT("Root has right flags"), ExplicitlyMuted && ExplicitlyMuted->Flags == EConcertReplicationMuteOption::ObjectAndSubobjects);
					
					TestTrue(TEXT("Subobject is unmuted"), ExplicitlyUnmuted != nullptr);
					TestTrue(TEXT("Subobject has right flags"), ExplicitlyUnmuted && ExplicitlyUnmuted->Flags == EConcertReplicationMuteOption::ObjectAndSubobjects);
					
					TestTrue(TEXT("NestedSubobject is unmuted"), Response.ImplicitlyUnmutedObjects.Contains(Replicator_BarNestedSubobject->TestObject));
				});
		});

		// Tests that involve FConcertReplication_ChangeStream_Request while using mutes
		Describe("Muting hierarchy (include changing stream structure)", [this]()
		{
			Describe("Keep hierarchy muted when root object is removed from one stream but referenced by another stream", [this]()
			{
				BeforeEach([this]()
				{
					ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderJoinArgs;
					SenderJoinArgs.Streams.Add(Replicator_Bar->CreateStream(StreamId_Bar));
					SenderJoinArgs.Streams.Add(Replicator_BarSubobject->CreateStream(StreamId_BarSubobject));
					
					Sender->JoinReplication(SenderJoinArgs);
					Sender->GetClientReplicationManager().MuteObjects({ Replicator_Bar->TestObject }, EConcertReplicationMuteOption::ObjectAndSubobjects);
				});

				AfterEach([this]()
				{
					Sender->GetClientReplicationManager()
						.QueryMuteState()
						.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
						{
							TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 1"), Response.ExplicitlyMutedObjects.Num(), 1);
							TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 0"), Response.ExplicitlyUnmutedObjects.Num(), 0);
							TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 1"), Response.ImplicitlyMutedObjects.Num(), 1);
							TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 0"), Response.ImplicitlyUnmutedObjects.Num(), 0);
							
							TestTrue(TEXT("ExplicitlyMutedObjects.Contains(Bar)"), Response.ExplicitlyMutedObjects.Contains(Replicator_Bar->TestObject));
							TestTrue(TEXT("ExplicitlyMutedObjects.Contains(Subobject)"), Response.ImplicitlyMutedObjects.Contains(Replicator_BarSubobject->TestObject));
						});
				});
				
				It("Single FConcertReplication_ChangeStream_Request for adding & removing root object", [this]()
				{
					FConcertReplication_ChangeStream_Request Request;
					Request.StreamsToRemove.Add(StreamId_Bar);
					Request.StreamsToAdd.Add(Replicator_Bar->CreateStream(FGuid::NewGuid()));
					Sender->GetClientReplicationManager().ChangeStream(Request);
				});
				
				It("Separate FConcertReplication_ChangeStream_Requests for adding & removing root object", [this]()
				{
					FConcertReplication_ChangeStream_Request InitialRequest;
					InitialRequest.StreamsToAdd.Add(Replicator_Bar->CreateStream(FGuid::NewGuid()));
					Sender->GetClientReplicationManager().ChangeStream(InitialRequest);
					
					FConcertReplication_ChangeStream_Request Request;
					Request.StreamsToRemove.Add(StreamId_Bar);
					Sender->GetClientReplicationManager().ChangeStream(Request);
				});
			});
			
			Describe("Removing multiple parents retains and cleans up state correctly.", [this]()
			{
				BeforeEach([this]()
				{
					ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderJoinArgs;
					SenderJoinArgs.Streams.Add(Replicator_Bar->CreateStream(StreamId_Bar));
					SenderJoinArgs.Streams.Add(Replicator_BarSubobject->CreateStream(StreamId_BarSubobject));
					SenderJoinArgs.Streams.Add(Replicator_BarNestedSubobject->CreateStream(StreamId_BarNestedSubobject));
				
					Sender->JoinReplication(SenderJoinArgs);
					Sender->GetClientReplicationManager().MuteObjects({ Replicator_Bar->TestObject }, EConcertReplicationMuteOption::ObjectAndSubobjects);
					Sender->GetClientReplicationManager().MuteObjects({ Replicator_BarSubobject->TestObject }, EConcertReplicationMuteOption::ObjectAndSubobjects);
				});

				Describe("Retain all mute state", [this]()
				{
					AfterEach([this]()
					{
						Sender->GetClientReplicationManager()
							.QueryMuteState()
							.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
							{
								TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 2);
								TestEqual(TEXT("ExplicitlyUnmutedObjects.Num()"), Response.ExplicitlyUnmutedObjects.Num(), 0);
								TestEqual(TEXT("ImplicitlyMutedObjects.Num()"), Response.ImplicitlyMutedObjects.Num(), 1);
								TestEqual(TEXT("ImplicitlyUnmutedObjects.Num()"), Response.ImplicitlyUnmutedObjects.Num(), 0);
							
								TestTrue(TEXT("ImplicitlyMutedObjects.Contains(Bar)"), Response.ExplicitlyMutedObjects.Contains(Replicator_Bar->TestObject));
								TestTrue(TEXT("ImplicitlyMutedObjects.Contains(BarSubobject)"), Response.ExplicitlyMutedObjects.Contains(Replicator_BarSubobject->TestObject));
								TestTrue(TEXT("ImplicitlyMutedObjects.Contains(BarNestedSubobject)"), Response.ImplicitlyMutedObjects.Contains(Replicator_BarNestedSubobject->TestObject));
							});
					});
					
					It("After removing middle parent", [this]()
					{
						const FConcertReplication_ChangeStream_Request Request { .StreamsToRemove = { StreamId_BarSubobject } };
						Sender->GetClientReplicationManager().ChangeStream(Request);
					});
					It("After removing root", [this]()
					{
						const FConcertReplication_ChangeStream_Request Request { .StreamsToRemove = { StreamId_Bar } };
						Sender->GetClientReplicationManager().ChangeStream(Request);
					});
					It("Remove root and parent in single request", [this]()
					{
						const FConcertReplication_ChangeStream_Request Request { .StreamsToRemove = { StreamId_Bar, StreamId_BarSubobject } };
						Sender->GetClientReplicationManager().ChangeStream(Request);
					});
					It("Remove root, then parent", [this]()
					{
						const FConcertReplication_ChangeStream_Request RemoveBar { .StreamsToRemove = { StreamId_Bar } };
						Sender->GetClientReplicationManager().ChangeStream(RemoveBar);

						const FConcertReplication_ChangeStream_Request RemoveBarSubobject { .StreamsToRemove = { StreamId_BarSubobject } };
						Sender->GetClientReplicationManager().ChangeStream(RemoveBarSubobject);
					});
					It ("Remove parent, then root", [this]()
					{
						const FConcertReplication_ChangeStream_Request RemoveBarSubobject { .StreamsToRemove = { StreamId_BarSubobject } };
						Sender->GetClientReplicationManager().ChangeStream(RemoveBarSubobject);
						
						const FConcertReplication_ChangeStream_Request RemoveBar { .StreamsToRemove = { StreamId_Bar } };
						Sender->GetClientReplicationManager().ChangeStream(RemoveBar);
					});
				});
				
				Describe("After unmuting root and parent, nothing retains mute state", [this]()
				{
					AfterEach([this]()
					{
						Sender->GetClientReplicationManager()
							.QueryMuteState()
							.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
							{
								TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 0"), Response.ExplicitlyMutedObjects.Num(), 0);
								TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 0"), Response.ExplicitlyUnmutedObjects.Num(), 0);
								TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 0"), Response.ImplicitlyMutedObjects.Num(), 0);
								TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 0"), Response.ImplicitlyUnmutedObjects.Num(), 0);
							});
					});

					It("Unmute root and parent in single request", [this]()
					{
						Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_Bar->TestObject, Replicator_BarSubobject->TestObject });
					});
					It("Unmute root, then parent", [this]()
					{
						Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_Bar->TestObject });
						Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_BarSubobject->TestObject });
					});
					It ("Unmute parent, then root", [this]()
					{
						Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_BarSubobject->TestObject });
						Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_Bar->TestObject });
					});
				});

				Describe("After removing root and parent, nothing retains mute state", [this]()
				{
					BeforeEach([this]()
					{
						// The nested subobject will keep alive the entire mute state if not removed because it causes its outers to be referenced.
						const FConcertReplication_ChangeStream_Request Request { .StreamsToRemove = { StreamId_BarNestedSubobject } };
						Sender->GetClientReplicationManager().ChangeStream(Request);
					});
					
					AfterEach([this]()
					{
						Sender->GetClientReplicationManager()
							.QueryMuteState()
							.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
							{
								TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 0"), Response.ExplicitlyMutedObjects.Num(), 0);
								TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 0"), Response.ExplicitlyUnmutedObjects.Num(), 0);
								TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 0"), Response.ImplicitlyMutedObjects.Num(), 0);
								TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 0"), Response.ImplicitlyUnmutedObjects.Num(), 0);
							});
					});
					
					It("Remove root and parent in single request", [this]()
					{
						const FConcertReplication_ChangeStream_Request Request { .StreamsToRemove = { StreamId_Bar, StreamId_BarSubobject } };
						Sender->GetClientReplicationManager().ChangeStream(Request);
					});
					It("Remove root, then parent", [this]()
					{
						const FConcertReplication_ChangeStream_Request RemoveBar { .StreamsToRemove = { StreamId_Bar } };
						Sender->GetClientReplicationManager().ChangeStream(RemoveBar);

						const FConcertReplication_ChangeStream_Request RemoveBarSubobject { .StreamsToRemove = { StreamId_BarSubobject } };
						Sender->GetClientReplicationManager().ChangeStream(RemoveBarSubobject);
					});
					It ("Remove parent, then root", [this]()
					{
						const FConcertReplication_ChangeStream_Request RemoveBarSubobject { .StreamsToRemove = { StreamId_BarSubobject } };
						Sender->GetClientReplicationManager().ChangeStream(RemoveBarSubobject);
						
						const FConcertReplication_ChangeStream_Request RemoveBar { .StreamsToRemove = { StreamId_Bar } };
						Sender->GetClientReplicationManager().ChangeStream(RemoveBar);
					});
				});
			});
		});
	}
}