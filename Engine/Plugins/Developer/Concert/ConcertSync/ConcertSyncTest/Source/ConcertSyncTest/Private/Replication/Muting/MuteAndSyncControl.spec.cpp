// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication
{
	BEGIN_DEFINE_SPEC(FMuteAndSyncControlSpec, "VirtualProduction.Concert.Replication.Muting", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	TSharedPtr<FReplicationServer> Server;
	FReplicationClient* Sender = nullptr;
	FReplicationClient* Receiver = nullptr;

	FGuid ObjectStreamId = FGuid::NewGuid();
	TSharedPtr<FObjectTestReplicator> ObjectReplicator;

	FObjectReplicationContext MakeSenderToReceiverContext() const { return { *Sender, *Server, *Receiver }; }
	END_DEFINE_SPEC(FMuteAndSyncControlSpec);

	/** This tests that muting and sync control interact correctly. */
	void FMuteAndSyncControlSpec::Define()
	{
		BeforeEach([this]
		{
			Server = MakeShared<FReplicationServer>(*this);
			Sender = &Server->ConnectClient();
			Receiver = &Server->ConnectClient();

			ObjectReplicator = MakeShared<FObjectTestReplicator>();
		});
		AfterEach([this]
		{
			// Test would hold onto this for rest of engine lifetime. Clean up this mini would-be leak.
			Server.Reset();
			ObjectReplicator.Reset();
			
			Sender = nullptr;
			Receiver = nullptr;
		});
		
		Describe("When client mutes / unmutes its own object", [this]()
		{
			BeforeEach([this]()
			{
				Sender->JoinReplication(ObjectReplicator->CreateSenderArgs(ObjectStreamId));
				// Tells client that the object is ready to replicate
				Sender->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
				// Tells server intent to replicate the object
				Sender->GetClientReplicationManager().TakeAuthorityOver({ ObjectReplicator->TestObject });

				// This gives sync control meaning that now the object can be replicated.
				Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });

				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(
					[this](const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event)
					{
						AddError(TEXT("Requesting client was not supposed to receive FConcertReplication_ChangeSyncControl"));
					});
				Sender->GetClientReplicationManager().MuteObjects({ ObjectReplicator->TestObject });
			});
			
			It("Client predicts new sync control for muted object", [this]()
			{
				TestTrue(TEXT("Has no sync control"), !Sender->GetClientReplicationManager().HasSyncControl({ ObjectStreamId, ObjectReplicator->TestObject }));
				
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { ObjectStreamId },
					[this](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
					{
						// If the client predicted the sync control correctly, it should not attempt to send to the server.
						AddError(TEXT("Sending client did not predict sync control correctly"));
					});
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});

			It("Client receives new sync control for unmuted object", [this]()
			{
				Sender->GetClientReplicationManager()
					.UnmuteObjects({ ObjectReplicator->TestObject })
					.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						TestTrue(TEXT("Response.IsSuccess()"), Response.IsSuccess());
						TestEqual(TEXT("Response.SyncControl.NewControlStates.Num() == 1"), Response.SyncControl.NewControlStates.Num(), 1);
						
						const bool* bIsEnabled = Response.SyncControl.NewControlStates.Find({ ObjectStreamId, ObjectReplicator->TestObject });
						TestTrue(TEXT("Object has regained sync control"), bIsEnabled && *bIsEnabled);
					});
				
				TestTrue(TEXT("Has sync control"), Sender->GetClientReplicationManager().HasSyncControl({ ObjectStreamId, ObjectReplicator->TestObject }));
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { ObjectStreamId });
				ObjectReplicator->TestValuesWereReplicated(*this);
			});
		});

		Describe("Clients receive FConcertReplication_ChangeSyncControl when other clients", [this]()
		{
			BeforeEach([this]()
			{
				Sender->JoinReplication(ObjectReplicator->CreateSenderArgs(ObjectStreamId));
				// Tells client that the object is ready to replicate
				Sender->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
				// Tells server intent to replicate the object
				Sender->GetClientReplicationManager().TakeAuthorityOver({ ObjectReplicator->TestObject });

				Receiver->JoinReplicationAsListener({});
			});
			
			It("Mute the object", [this]()
			{
				bool bReceivedEvent = false;
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(
					[this, &bReceivedEvent](const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event)
					{
						bReceivedEvent = true;
						TestEqual(TEXT("Event.NewControlStates.Num() == 1"), Event.NewControlStates.Num(), 1);

						const bool* bState = Event.NewControlStates.Find({ ObjectStreamId, ObjectReplicator->TestObject });
						TestTrue(TEXT("Lost sync control"), bState && !*bState);
					});
				
				Receiver->GetClientReplicationManager().MuteObjects({ ObjectReplicator->TestObject });
				TestTrue(TEXT("Received event"), bReceivedEvent);
			});

			It("Unmute the object", [this]()
			{
				Receiver->GetClientReplicationManager().MuteObjects({ ObjectReplicator->TestObject });
				
				bool bReceivedEvent = false;
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(
					[this, &bReceivedEvent](const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event)
					{
						bReceivedEvent = true;
						TestEqual(TEXT("Event.NewControlStates.Num() == 1"), Event.NewControlStates.Num(), 1);

						const bool* bState = Event.NewControlStates.Find({ ObjectStreamId, ObjectReplicator->TestObject });
						TestTrue(TEXT("Gained sync control"), bState && *bState);
					});
				
				Receiver->GetClientReplicationManager()
					.UnmuteObjects({ ObjectReplicator->TestObject })
					.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						TestTrue(TEXT("Success"), Response.IsSuccess());
						// Receiver has nothing registered. Make sure it does not receive the data intended for Sender.
						TestTrue(TEXT("SyncControl.IsEmpty()"), Response.SyncControl.IsEmpty());
					});
				TestTrue(TEXT("Received event"), bReceivedEvent);
			});
		});
	}
}