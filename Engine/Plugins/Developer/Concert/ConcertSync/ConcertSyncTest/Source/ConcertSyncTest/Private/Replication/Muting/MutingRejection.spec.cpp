// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication
{
	/** Tests that the rejection cases outlined in FConcertReplication_ChangeMuteState_Request work. */
	BEGIN_DEFINE_SPEC(FMutingRejectionSpec, "VirtualProduction.Concert.Replication.Muting", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TSharedPtr<FReplicationServer> Server;
		FReplicationClient* Sender = nullptr;
		FReplicationClient* Receiver = nullptr;
	
		// Leverage FObjectTestReplicator to create more UObjects
		TSharedPtr<FObjectTestReplicator> ObjectReplicator_Foo;			
		TSharedPtr<FObjectTestReplicator> Replicator_FooSubobject;

		void ValidateRejection(const FConcertReplication_ChangeMuteState_Response& Response)
		{
			TestTrue(TEXT("Request failed"), Response.IsFailure());
			TestTrue(TEXT("Request rejected"), Response.ErrorCode == EConcertReplicationMuteErrorCode::Rejected);
			TestEqual(TEXT("RejectionReason.Num() == 1"), Response.RejectionReasons.Num(), 1);
			TestTrue(TEXT("RejectionReasons.Contains(UnregisteredObject)"), Response.RejectionReasons.Contains(ObjectReplicator_Foo->TestObject));
		}
	END_DEFINE_SPEC(FMutingRejectionSpec);

	void FMutingRejectionSpec::Define()
	{
		BeforeEach([this]
		{
			Server		= MakeShared<FReplicationServer>(*this);
			Sender		= &Server->ConnectClient();
			Receiver	= &Server->ConnectClient();

			ObjectReplicator_Foo			= MakeShared<FObjectTestReplicator>();
			Replicator_FooSubobject			= ObjectReplicator_Foo->CreateSubobjectReplicator();
		});
		AfterEach([this]
		{
			// Test would hold onto this for rest of engine lifetime. Clean up this mini would-be leak.
			Server.Reset();
			ObjectReplicator_Foo.Reset();
			Replicator_FooSubobject.Reset();
		});

		Describe("Request rejection cases", [this]()
		{
			It("Reject: Explicitly muting unreferenced object (EConcertReplicationMuteFlags::None)", [this]()
			{
				Sender->JoinReplication();
				Sender->GetClientReplicationManager()
					.MuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::OnlyObject)
					.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						ValidateRejection(Response);
					});
			});

			It("Reject: Explicitly unmuting unreferenced object (EConcertReplicationMuteFlags::None)", [this]()
			{
				Sender->JoinReplication();
				Sender->GetClientReplicationManager()
					.UnmuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::OnlyObject)
					.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						ValidateRejection(Response);
					});
			});

			It("Reject: Explicitly muting unreferenced object (EConcertReplicationMuteFlags::ObjectAndSubobjects) without subobject", [this]()
			{
				Sender->JoinReplication();
				Sender->GetClientReplicationManager()
					.MuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::ObjectAndSubobjects)
					.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						ValidateRejection(Response);
					});
			});

			It("Reject: Explicitly unmuting unreferenced object (EConcertReplicationMuteFlags::ObjectAndSubobjects) without subobject", [this]()
			{
				Sender->JoinReplication();
				Sender->GetClientReplicationManager()
					.UnmuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::ObjectAndSubobjects)
					.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						ValidateRejection(Response);
					});
			});
			
			It("Allow: Explicitly muting unreferenced object (EConcertReplicationMuteFlags::ObjectAndSubobjects) with subobject", [this]()
			{
				Sender->JoinReplication(Replicator_FooSubobject->CreateSenderArgs());
				Sender->GetClientReplicationManager()
					.MuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::ObjectAndSubobjects)
					.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						TestTrue(TEXT("Success"), Response.IsSuccess());
					});

				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestEqual(TEXT("ExplicitlyMutedObjects.Num() == 1"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num() == 0"), Response.ExplicitlyUnmutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num() == 1"), Response.ImplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num() == 0"), Response.ImplicitlyUnmutedObjects.Num(), 0);

						const FConcertReplication_ObjectMuteSetting* FooMuteSetting = Response.ExplicitlyMutedObjects.Find(ObjectReplicator_Foo->TestObject);
						TestTrue(TEXT("ExplicitlyMutedObjects.Contains(Subobject)"), FooMuteSetting && FooMuteSetting->Flags == EConcertReplicationMuteOption::ObjectAndSubobjects);
						TestTrue(TEXT("ImplicitlyMutedObjects.Contains(Subobject)"), Response.ImplicitlyMutedObjects.Contains(Replicator_FooSubobject->TestObject));
					});
			});
			
			It("Allow: Explicitly unmuting unreferenced object (EConcertReplicationMuteFlags::ObjectAndSubobjects) with subobject", [this]()
			{
				Sender->JoinReplication(Replicator_FooSubobject->CreateSenderArgs());

				// Check nothing is muted if ...
				Sender->GetClientReplicationManager().MuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::ObjectAndSubobjects);
				// ... we unmute with EConcertReplicationMuteFlags::ObjectAndSubobjects flag
				Sender->GetClientReplicationManager().UnmuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::ObjectAndSubobjects);
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestTrue(TEXT("Nothing  muted"), Response.IsEmpty());
					});

				// Check nothing is muted if ...
				Sender->GetClientReplicationManager().MuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::ObjectAndSubobjects);
				// ... we unmute with EConcertReplicationMuteFlags::None flag
				Sender->GetClientReplicationManager().UnmuteObjects({ ObjectReplicator_Foo->TestObject}, EConcertReplicationMuteOption::OnlyObject);
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestTrue(TEXT("Nothing  muted"), Response.IsEmpty());
					});
			});

			It("Reject: Mute and unmute in same opereation", [this]()
			{
				Sender->JoinReplication(Replicator_FooSubobject->CreateSenderArgs());
				FConcertReplication_ChangeMuteState_Request Request;
				Request.ObjectsToMute.Add(ObjectReplicator_Foo->TestObject);
				Request.ObjectsToUnmute.Add(ObjectReplicator_Foo->TestObject);
				Sender->GetClientReplicationManager()
					.ChangeMuteState(Request)
					.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						ValidateRejection(Response);
					});
			});
		});
	}
	
	/** This tests that muting requests are not served if EConcertSyncSessionFlags::ShouldAllowGlobalMuting is not set. */
	BEGIN_DEFINE_SPEC(FMutingWithoutFlagSpec, "VirtualProduction.Concert.Replication.Muting", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TSharedPtr<FReplicationServer> Server;
		FReplicationClient* Sender = nullptr;

		TSharedPtr<FObjectTestReplicator> ObjectReplicator;
	END_DEFINE_SPEC(FMutingWithoutFlagSpec);

	void FMutingWithoutFlagSpec::Define()
	{
		BeforeEach([this]
		{
			Server = MakeShared<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession & ~EConcertSyncSessionFlags::ShouldAllowGlobalMuting);
			Sender = &Server->ConnectClient();
			ObjectReplicator = MakeShared<FObjectTestReplicator>();
			
			Sender->JoinReplication(ObjectReplicator->CreateSenderArgs());
		});
		AfterEach([this]
		{
			// Test would hold onto this for rest of engine lifetime. Clean up this mini would-be leak.
			Server.Reset();
			ObjectReplicator.Reset();
		});

		Describe("When EConcertSyncSessionFlags::ShouldAllowGlobalMuting is not set", [this]()
		{
			Describe("Mute request is not sent via IConcertClientReplicationManager API", [this]()
			{
				It("FConcertReplication_ChangeMuteState_Request is not sent to server", [this]
				{
					Server->GetServerSessionMock()->RegisterCustomRequestHandler<FConcertReplication_ChangeMuteState_Request, FConcertReplication_ChangeMuteState_Response>(
						[this](const FConcertSessionContext&, const FConcertReplication_ChangeMuteState_Request&, FConcertReplication_ChangeMuteState_Response&)
						{
							AddError(TEXT("Client was not supposed to send the request to the server."));
							return EConcertSessionResponseCode::InvalidRequest;
						});
					
					Sender->GetClientReplicationManager().MuteObjects({ ObjectReplicator->TestObject })
						.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
						{
							TestTrue(TEXT("Request rejected"), Response.ErrorCode == EConcertReplicationMuteErrorCode::Rejected);
						});
				});
				
				It("FConcertReplication_QueryMuteState_Request is not sent to server", [this]
				{
					Server->GetServerSessionMock()->RegisterCustomRequestHandler<FConcertReplication_QueryMuteState_Request, FConcertReplication_QueryMuteState_Response>(
						[this](const FConcertSessionContext&, const FConcertReplication_QueryMuteState_Request&, FConcertReplication_QueryMuteState_Response&)
						{
							AddError(TEXT("Client was not supposed to send the request to the server."));
							return EConcertSessionResponseCode::InvalidRequest;
						});
					
					Sender->GetClientReplicationManager().QueryMuteState()
						.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
						{
							TestTrue(TEXT("Response.IsEmpty()"), Response.IsEmpty());
						});
				});
			});
			
			Describe("Server rejects", [this]()
			{
				It("FConcertReplication_ChangeMuteState_Request", [this]
				{
					FConcertReplication_ChangeMuteState_Request Request;
					Request.ObjectsToMute.Add(ObjectReplicator->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects });
					
					Sender->GetClientSessionMock()
						->SendCustomRequest<FConcertReplication_ChangeMuteState_Request, FConcertReplication_ChangeMuteState_Response>(Request, Sender->GetClientSessionMock()->GetSessionServerEndpointId())
						.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
						{
							TestTrue(TEXT("Response.IsFailure()"), Response.IsFailure());
							// The server's request handler returns EConcertSessionResponseCode::Failed, which makes Concert default construct the response hence timeout
							TestTrue(TEXT("Response.ErrorCode == Timeout"), Response.ErrorCode == EConcertReplicationMuteErrorCode::Timeout);
						});
				});

				It("FConcertReplication_QueryMuteState_Request", [this]
				{
					FConcertReplication_QueryMuteState_Request Request;
					
					Sender->GetClientSessionMock()
						->SendCustomRequest<FConcertReplication_QueryMuteState_Request, FConcertReplication_QueryMuteState_Response>(Request, Sender->GetClientSessionMock()->GetSessionServerEndpointId())
						.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
						{
							// The server's request handler returns EConcertSessionResponseCode::Failed, which makes Concert default construct the response
							TestTrue(TEXT("Response.IsFailure()"), Response.IsFailure());
							TestTrue(TEXT("Response.IsEmpty()"), Response.IsEmpty());
						});
				});
			});
		});
	}
}