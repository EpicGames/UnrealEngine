// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication
{
	/** This tests that replication works correctly after muting & unmuting. */
	BEGIN_DEFINE_SPEC(FMutingReplicationSpec, "VirtualProduction.Concert.Replication.Muting", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	TSharedPtr<FReplicationServer> Server;
	FReplicationClient* Sender = nullptr;
	FReplicationClient* Receiver = nullptr;

	FGuid StreamId_Foo					= FGuid::NewGuid();
	FGuid StreamId_FooSubobject			= FGuid::NewGuid();
	FGuid StreamId_Bar					= FGuid::NewGuid();
	FGuid StreamId_BarSubobject			= FGuid::NewGuid();
	FGuid StreamId_BarNestedSubobject	= FGuid::NewGuid();
	// Leverage FObjectTestReplicator to create more UObjects
	TSharedPtr<FObjectTestReplicator> Replicator_Foo;			
	TSharedPtr<FObjectTestReplicator> Replicator_FooSubobject;		
	TSharedPtr<FObjectTestReplicator> Replicator_Bar;			
	TSharedPtr<FObjectTestReplicator> Replicator_BarSubobject;		
	TSharedPtr<FObjectTestReplicator> Replicator_BarNestedSubobject;

	FObjectReplicationContext MakeSenderToReceiverContext(const TCHAR* Context = nullptr) const { return { *Sender, *Server, *Receiver, Context }; }
	END_DEFINE_SPEC(FMutingReplicationSpec);

	/** This tests that muting requests work when EConcertSyncSessionFlags::ShouldAllowGlobalMuting is set. */
	void FMutingReplicationSpec::Define()
	{
		BeforeEach([this]
		{
			Server		= MakeShared<FReplicationServer>(*this);
			Sender		= &Server->ConnectClient();
			Receiver	= &Server->ConnectClient();

			Replicator_Foo					= MakeShared<FObjectTestReplicator>(TEXT("Foo"));
			Replicator_FooSubobject			= Replicator_Foo->CreateSubobjectReplicator(TEXT("FooSubobject"));
			Replicator_Bar					= MakeShared<FObjectTestReplicator>(TEXT("Bar"));
			Replicator_BarSubobject			= Replicator_Bar->CreateSubobjectReplicator(TEXT("BarSubobject"));
			Replicator_BarNestedSubobject	= Replicator_BarSubobject->CreateSubobjectReplicator(TEXT("BarNestedSubobject"));
		});
		AfterEach([this]
		{
			// Test would hold onto this for rest of engine lifetime. Clean up this mini would-be leak.
			Server.Reset();
			Replicator_Foo.Reset();
			Replicator_FooSubobject.Reset();
			Replicator_Bar.Reset();
			Replicator_BarSubobject.Reset();
			Replicator_BarNestedSubobject.Reset();
		});

		Describe("Replication with mute & unmute", [this]()
		{
			BeforeEach([this]()
			{
				ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderJoinArgs;
				SenderJoinArgs.Streams.Add(Replicator_Foo->CreateStream(StreamId_Foo));
				SenderJoinArgs.Streams.Add(Replicator_FooSubobject->CreateStream(StreamId_FooSubobject));
				SenderJoinArgs.Streams.Add(Replicator_Bar->CreateStream(StreamId_Bar));
				SenderJoinArgs.Streams.Add(Replicator_BarSubobject->CreateStream(StreamId_BarSubobject));
				SenderJoinArgs.Streams.Add(Replicator_BarNestedSubobject->CreateStream(StreamId_BarNestedSubobject));
				
				Sender->JoinReplication(SenderJoinArgs);
				// Tells server intent to replicate the object
				Sender->GetClientReplicationManager().TakeAuthorityOver({
					Replicator_Foo->TestObject,
					Replicator_FooSubobject->TestObject,
					Replicator_Bar->TestObject,
					Replicator_BarSubobject->TestObject,
					Replicator_BarNestedSubobject->TestObject
					});

				// This gives sync control meaning that now the object can be replicated.
				Receiver->JoinReplicationAsListener({ Replicator_Foo->TestObject, Replicator_FooSubobject->TestObject, Replicator_Bar->TestObject, Replicator_BarSubobject->TestObject, Replicator_BarNestedSubobject->TestObject });

				bool bReceivedEvent = false;
				FConcertReplication_ChangeMuteState_Request MuteRequest;
				MuteRequest.ObjectsToMute.Add(Replicator_Foo->TestObject, { EConcertReplicationMuteOption::ObjectAndSubobjects });
				MuteRequest.ObjectsToMute.Add(Replicator_Bar->TestObject, { EConcertReplicationMuteOption::OnlyObject });
				Sender->GetClientReplicationManager()
					.ChangeMuteState(MuteRequest)
					.Next([this, &bReceivedEvent](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						bReceivedEvent = true;
						TestTrue(TEXT("Mute request successful"), Response.IsSuccess());
					});
				TestTrue(TEXT("Mute response received"), bReceivedEvent);
			});

			Describe("Don't replicate: Muted objects", [this]()
			{
				It("Foo was not replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_Foo->TestObject);
					Replicator_Foo->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_Foo });
					Replicator_Foo->TestValuesWereNotReplicated(*this);
				});
				
				It("FooSubobject was not replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_FooSubobject->TestObject);
					Replicator_FooSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_FooSubobject });
					Replicator_FooSubobject->TestValuesWereNotReplicated(*this);
				});
				It("Bar was not replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_Bar->TestObject);
					Replicator_Bar->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_Bar });
					Replicator_Bar->TestValuesWereNotReplicated(*this);
				});
				
				// This subobject is not affected
				It("BarSubobject was replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_BarSubobject->TestObject);
					Replicator_BarSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_BarSubobject });
					Replicator_BarSubobject->TestValuesWereReplicated(*this);
				});
			});
			
			Describe("Replicate: Muted objects after unmuting", [this]()
			{
				BeforeEach([this]()
				{
					bool bReceivedEvent = false;
					Sender->GetClientReplicationManager()
						.UnmuteObjects({ Replicator_Foo->TestObject })
						.Next([this, &bReceivedEvent](FConcertReplication_ChangeMuteState_Response&& Response)
						{
							bReceivedEvent = true;
							TestTrue(TEXT("Unmute request successful"), Response.ErrorCode == EConcertReplicationMuteErrorCode::Accepted);
						});
					TestTrue(TEXT("Mute response received"), bReceivedEvent);
				});
				
				It("Foo was replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_Foo->TestObject);
					Replicator_Foo->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_Foo });
					Replicator_Foo->TestValuesWereReplicated(*this);
				});
				
				It("FooSubobject was replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_FooSubobject->TestObject);
					Replicator_FooSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_FooSubobject });
					Replicator_FooSubobject->TestValuesWereReplicated(*this);
				});
				It("Bar was not replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_Bar->TestObject);
					Replicator_Bar->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_Bar });
					Replicator_Bar->TestValuesWereNotReplicated(*this);
				});
				
				// This subobject is not affected
				It("BarSubobject was replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_BarSubobject->TestObject);
					Replicator_BarSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_BarSubobject });
					Replicator_BarSubobject->TestValuesWereReplicated(*this);
				});
			});

			It("Replicate: subobject when unmuted explicitly", [this]()
			{
				Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_Foo->TestObject }, EConcertReplicationMuteOption::OnlyObject);
				
				Sender->GetBridgeMock().InjectAvailableObject(*Replicator_FooSubobject->TestObject);
				Replicator_FooSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_FooSubobject });
				Replicator_FooSubobject->TestValuesWereReplicated(*this);
			});

			Describe("Replicate: remove stream that contain muted parent.", [this]()
			{
				BeforeEach([this]()
				{
					const FConcertReplication_ChangeStream_Request Request { .StreamsToRemove = { StreamId_Foo } };
					Sender->GetClientReplicationManager().ChangeStream(Request);
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_FooSubobject->TestObject);
				});

				It("Don't replicate: subobject stays implicitly muted", [this]()
				{
					Replicator_FooSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_FooSubobject });
					Replicator_FooSubobject->TestValuesWereNotReplicated(*this);
				});

				It("Replicate: Explicitly unmuting parent still unmutes children", [this]()
				{
					Sender->GetClientReplicationManager().UnmuteObjects({ Replicator_Foo->TestObject });
					
					Replicator_FooSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_FooSubobject });
					Replicator_FooSubobject->TestValuesWereReplicated(*this);
				});
			});

			Describe("Replicate: implicitly unmuted objects", [this]()
			{
				BeforeEach([this]()
				{
					IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
					ReplicationManager.MuteObjects({ Replicator_Bar->TestObject }, EConcertReplicationMuteOption::ObjectAndSubobjects);
					ReplicationManager.UnmuteObjects({ Replicator_BarSubobject->TestObject }, EConcertReplicationMuteOption::ObjectAndSubobjects);
				});

				// Explicitly muted	 > does not replicate
				It("Bar was not replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_Bar->TestObject);
					Replicator_Bar->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_Bar });
					Replicator_Bar->TestValuesWereNotReplicated(*this);
				});
				
				// Explicitly unmuted > does replicate
				It("BarSubobject was replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_BarSubobject->TestObject);
					Replicator_BarSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_BarSubobject });
					Replicator_BarSubobject->TestValuesWereReplicated(*this);
				});

				// Implicitly unmuted > does replicate
				It("BarNestedSubobject was replicated", [this]()
				{
					Sender->GetBridgeMock().InjectAvailableObject(*Replicator_BarNestedSubobject->TestObject);
					Replicator_BarNestedSubobject->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { StreamId_BarNestedSubobject });
					Replicator_BarNestedSubobject->TestValuesWereReplicated(*this);
				});
			});
		});
	}
}