// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication
{
	/** Tests that EConcertReplicationMuteRequestFlags::ResetMuteState works as intended in conjunction with FConcertReplication_ChangeMuteState_Request. */
	BEGIN_DEFINE_SPEC(FResetMuteStateSpec, "VirtualProduction.Concert.Replication.Muting.Reset", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;
		TUniquePtr<FObjectTestReplicator> SecondObjectReplicator;

		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Sender = nullptr;
		FReplicationClient* Receiver = nullptr;

		const FGuid StreamId = FGuid::NewGuid();
	END_DEFINE_SPEC(FResetMuteStateSpec);

	void FResetMuteStateSpec::Define()
	{
		BeforeEach([this]
		{
			ObjectReplicator		= MakeUnique<FObjectTestReplicator>(TEXT("ObjectReplicator"));
			SecondObjectReplicator	= MakeUnique<FObjectTestReplicator>(TEXT("SecondObjectReplicator"));
			Server					= MakeUnique<FReplicationServer>(*this);
			Sender					= &Server->ConnectClient();
			Receiver				= &Server->ConnectClient();

			Sender->JoinReplication(ObjectReplicator->CreateSenderArgs(StreamId));
			Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });
			
			Sender->GetClientReplicationManager().TakeAuthorityOver({  ObjectReplicator->TestObject });
			Sender->GetClientReplicationManager().ChangeMuteState({ .ObjectsToMute = { { ObjectReplicator->TestObject, {} } } });
			// Needs to be known by server, so it can be muted
			Receiver->GetClientReplicationManager().ChangeStream({ .StreamsToAdd = { { SecondObjectReplicator->CreateStream(StreamId) } } });
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
		});

		It("When EConcertReplicationMuteRequestFlags::ClearMuteState is specified and ObjectsToUnmute is non-empty, the request is rejected.", [this]
		{
			int32 EventCount = 0;
			Sender->GetClientReplicationManager()
				.ChangeMuteState({ .Flags = EConcertReplicationMuteRequestFlags::ClearMuteState, .ObjectsToUnmute = { { ObjectReplicator->TestObject, {} } } })
				.Next([this, &EventCount](FConcertReplication_ChangeMuteState_Response&& Response)
				{
					++EventCount;
					TestEqual(TEXT("Success"), Response.ErrorCode, EConcertReplicationMuteErrorCode::Rejected);
					return Response;
				});
			TestEqual(TEXT("EventCount"), EventCount, 1);
		});

		Describe("When EConcertReplicationMuteRequestFlags::ResetMuteState is used on its own", [this]
		{
			const auto RunTest = [this]()
			{
				int32 EventCount = 0;
				TFuture<FConcertReplication_ChangeMuteState_Response> Future = Sender->GetClientReplicationManager()
					.ChangeMuteState({ .Flags = EConcertReplicationMuteRequestFlags::ClearMuteState })
					.Next([this, &EventCount](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						++EventCount;
						TestTrue(TEXT("Success"), Response.IsSuccess());
						return Response;
					});
				TestEqual(TEXT("EventCount"), EventCount, 1);
				return Future;
			};

			It("The response contains sync control", [this, RunTest]
			{
				RunTest()
				.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
				{
					TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 1);
					const bool* NewState = Response.SyncControl.NewControlStates.Find({ StreamId, ObjectReplicator->TestObject });
					TestTrue(TEXT("Has Sync Control"), NewState && *NewState);
				});
			});
			It("Server mute state is empty", [this, RunTest]
			{
				RunTest();
				
				int32 EventCount = 0;
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this, &EventCount](FConcertReplication_QueryMuteState_Response&& Response)
					{
						++EventCount;
						TestTrue(TEXT("IsEmpty"), Response.IsEmpty());
					});
				TestEqual(TEXT("EventCount"), EventCount, 1);
			});
		});

		Describe("When EConcertReplicationMuteRequestFlags::ResetMuteState is used and ObjectsToMute mutes the same object again", [this]
		{
			const auto RunTest = [this]()
			{
				int32 EventCount = 0;
				TFuture<FConcertReplication_ChangeMuteState_Response> Future = Sender->GetClientReplicationManager()
					.ChangeMuteState({ .Flags = EConcertReplicationMuteRequestFlags::ClearMuteState, .ObjectsToMute = { { ObjectReplicator->TestObject, {} } } })
					.Next([this, &EventCount](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						++EventCount;
						TestTrue(TEXT("Success"), Response.IsSuccess());
						return Response;
					});
				TestEqual(TEXT("EventCount"), EventCount, 1);
				return Future;
			};

			It("The response contains no sync control", [this, RunTest]
			{
				RunTest()
				.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
				{
					TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 0);
				});
			});
			It("Server mute state is set", [this, RunTest]
			{
				RunTest();
				
				int32 EventCount = 0;
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this, &EventCount](FConcertReplication_QueryMuteState_Response&& Response)
					{
						++EventCount;
						TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num()"), Response.ExplicitlyUnmutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num()"), Response.ImplicitlyMutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num()"), Response.ImplicitlyUnmutedObjects.Num(), 0);
						TestTrue(TEXT("Object is muted"), Response.ExplicitlyMutedObjects.Contains(ObjectReplicator->TestObject));
					});
				TestEqual(TEXT("EventCount"), EventCount, 1);
			});
		});
		
		Describe("When EConcertReplicationMuteRequestFlags::ResetMuteState is used and ObjectsToMute mutes another object", [this]
		{
			const auto RunTest = [this]()
			{
				int32 EventCount = 0;
				TFuture<FConcertReplication_ChangeMuteState_Response> Future = Sender->GetClientReplicationManager()
					.ChangeMuteState({ .Flags = EConcertReplicationMuteRequestFlags::ClearMuteState, .ObjectsToMute = { { SecondObjectReplicator->TestObject, {} } } })
					.Next([this, &EventCount](FConcertReplication_ChangeMuteState_Response&& Response)
					{
						++EventCount;
						TestTrue(TEXT("Success"), Response.IsSuccess());
						return Response;
					});
				TestEqual(TEXT("EventCount"), EventCount, 1);
				return Future;
			};

			It("The response contains sync control", [this, RunTest]
			{
				RunTest()
				.Next([this](FConcertReplication_ChangeMuteState_Response&& Response)
				{
					TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 1);
					const bool* NewState = Response.SyncControl.NewControlStates.Find({ StreamId, ObjectReplicator->TestObject });
					TestTrue(TEXT("Has Sync Control"), NewState && *NewState);
				});
			});
			It("Server mute state is set", [this, RunTest]
			{
				RunTest();
				
				int32 EventCount = 0;
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this, &EventCount](FConcertReplication_QueryMuteState_Response&& Response)
					{
						++EventCount;
						TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num()"), Response.ExplicitlyUnmutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num()"), Response.ImplicitlyMutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num()"), Response.ImplicitlyUnmutedObjects.Num(), 0);
						TestTrue(TEXT("Object is muted"), Response.ExplicitlyMutedObjects.Contains(SecondObjectReplicator->TestObject));
					});
				TestEqual(TEXT("EventCount"), EventCount, 1);
			});
		});
	}
}