// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Messages/ChangeClientEvent.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::ChangeClients
{
	BEGIN_DEFINE_SPEC(FChangeClientEventSpec, "VirtualProduction.Concert.Replication.PutState.Event", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;

		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;

		const FGuid StreamId = FGuid::NewGuid();
		FConcertReplicationStream StreamData;
	END_DEFINE_SPEC(FChangeClientEventSpec);

	/** This tests that FConcertReplication_ChangeClientEvent is correctly applied by the client when received. */
	void FChangeClientEventSpec::Define()
	{
		BeforeEach([this]
		{
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Client = &Server->ConnectClient();

			Client->JoinReplication();
			StreamData = ObjectReplicator->CreateStream(StreamId);
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
		});

		Describe("When client receives ChangeClientEvent", [this]
		{
			BeforeEach([this]
			{
				// This simulates the server sending to client - but it does not actually set any server state.
				// This would not actually happen in a real world use case.
				FConcertReplication_ChangeClientEvent Event;
				Event.ChangeData.StreamChange.StreamsToAdd.Add(StreamData);
				Event.ChangeData.AuthorityChange.TakeAuthority.Add({ ObjectReplicator->TestObject }, { { StreamId } });
				Event.ChangeData.SyncControlChange.NewControlStates.Add({ StreamId, ObjectReplicator->TestObject }, true);
				Server->GetServerSessionMock()->SendCustomEvent(Event, Client->GetEndpointId(), EConcertMessageFlags::ReliableOrdered);
			});

			It("Local client state is correct", [this]
			{
				const IConcertClientReplicationManager& ReplicationManager = Client->GetClientReplicationManager();
				
				const TArray<FConcertReplicationStream> Streams = ReplicationManager.GetRegisteredStreams();
				if (Streams.Num() != 1)
				{
					AddError(TEXT("Expected 1 stream"));
					return;
				}
				TestEqual(TEXT("Stream content"), Streams[0], StreamData);

				const TMap<FSoftObjectPath, TSet<FGuid>> AuthorityObjects = ReplicationManager.GetClientOwnedObjects();
				const TSet<FGuid>* AuthorityStreams = AuthorityObjects.Find(ObjectReplicator->TestObject);
				if (!AuthorityStreams)
				{
					AddError(TEXT("Expected authority"));
					return;
				}
				TestEqual(TEXT("AuthorityStreams.Num()"), AuthorityStreams->Num(), 1);
				TestTrue(TEXT("AuthorityStreams.Contains(StreamId)"), AuthorityStreams->Contains(StreamId));
				TestEqual(TEXT("AuthorityObjects.Num()"), AuthorityObjects.Num(), 1);

				const TSet<FConcertObjectInStreamID> SyncControl = ReplicationManager.GetSyncControlledObjects();
				TestEqual(TEXT("SyncControl.Num()"), SyncControl.Num(), 1);
				TestTrue(TEXT("SyncControl.Contains(TestObject)"), SyncControl.Contains({ StreamId, ObjectReplicator->TestObject }));
			});

			It("Attempts to replicate", [this]
			{
				FReplicationClient& Receiver = Server->ConnectClient();
				// Receiver won't actually receive anything because server will block it.
				Receiver.JoinReplicationAsListener({});

				bool bSentDataToServer = false;
				Client->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
				ObjectReplicator->SimulateSendObjectToReceiver(
					*this,
					FObjectReplicationContext{ *Client, *Server, Receiver },
					{ StreamId },
					[&bSentDataToServer](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
					{
						bSentDataToServer = true;
					},
					[this](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
					{
						// The server is supposed to reject the data because our tests did not send the FConcertReplication_ChangeClientEvent through the replication system.
						AddError(TEXT("Server forwarded the replicated data!"));
					});
				TestTrue(TEXT("Client attempted to replicate"), bSentDataToServer);
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});
		});
	}
}
