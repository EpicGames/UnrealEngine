// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Messages/ChangeClientEvent.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Replication/ReplicationTestInterface.h"
#include "Util/ScopedSessionDatabase.h"

#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::ChangeClients
{
	BEGIN_DEFINE_SPEC(FPutStateActivitiesSpec, "VirtualProduction.Concert.Replication.PutState.Activity", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FScopedSessionDatabase> SessionDatabase;
		TSharedPtr<ConcertSyncServer::Replication::IReplicationWorkspace> ReplicationWorkspace;
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;
		
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;

		const FGuid StreamId = FGuid::NewGuid();
		/**
		 * This is the max activity ID in the database after BeforeEach has executed.
		 * Update this value and doc string if you make a change in the future.
		 * For now, there are no activities produced for joining.
		 */
		const int32 ExpectedIdOfFirstActivityProducedByTest = 0;
		
		FReplicationClient& ConnectClient()
		{
			FReplicationClient& CreatedClient = Server->ConnectClient();
			SessionDatabase->SetEndpoint(CreatedClient.GetEndpointId(), { CreatedClient.GetClientInfo() });
			return CreatedClient;
		}
	END_DEFINE_SPEC(FPutStateActivitiesSpec);

	/** This tests that the correct activities are produced when FConcertReplication_PutState_Request is used. */
	void FPutStateActivitiesSpec::Define()
	{
		BeforeEach([this]
		{
			SessionDatabase = MakeUnique<FScopedSessionDatabase>(*this);
			const auto FindSessionClient = [this](const FGuid& EndpointId) -> TOptional<FConcertSessionClientInfo>
			{
				FConcertSessionClientInfo Info;
				const bool bFound = Server->GetServerSessionMock()->FindSessionClient(EndpointId, Info);
				return bFound ? Info : TOptional<FConcertSessionClientInfo>{};
			};
			const auto ShouldIgnoreClientActivityOnRestore = [](const FGuid& EndpointId){ return false; };
			ReplicationWorkspace = ConcertSyncServer::TestInterface::CreateReplicationWorkspace(*SessionDatabase, FindSessionClient, ShouldIgnoreClientActivityOnRestore);
			
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession, ReplicationWorkspace.ToSharedRef());
			Client = &ConnectClient();

			Client->JoinReplication(ObjectReplicator->CreateSenderArgs());
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
			SessionDatabase.Reset();
			ReplicationWorkspace.Reset();
		});

		// Since FConcertReplication_PutState_Request leverages internal systems, like muting, those systems may produce an activity as side effect:
		// validate that the put state implementation prevents that from happening. 
		It("No unexpected activities are produced", [this]
		{
			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Client->GetEndpointId(), { { ObjectReplicator->CreateStream(StreamId) } });
			Request.NewAuthorityState.Add(Client->GetEndpointId(), { TArray{ FConcertObjectInStreamID{ StreamId, ObjectReplicator->TestObject } } });
			Request.MuteChange.ObjectsToMute.Add(ObjectReplicator->TestObject);
			
			int32 EventCount = 0;
			Client->GetClientReplicationManager()
				.PutClientState(Request)
				.Next([this, &EventCount](FConcertReplication_PutState_Response&& Response)
				{
					++EventCount;
				});
			TestEqual(TEXT("EventCount"), EventCount, 1);
			
			TArray<FConcertSyncReplicationActivity> ActualActivities;
			ReplicationWorkspace->EnumerateReplicationActivities([&ActualActivities](const FConcertSyncReplicationActivity& Activity)
			{
				ActualActivities.Add(Activity);
				return EBreakBehavior::Continue;
			});

			// This number might change in the future, if you change PutRequest to produce more activities yourself. Update this test accordingly in that case.
			TestEqual(TEXT("ActualActivities.Num()"), ActualActivities.Num(), 0);
		});
	}
}
