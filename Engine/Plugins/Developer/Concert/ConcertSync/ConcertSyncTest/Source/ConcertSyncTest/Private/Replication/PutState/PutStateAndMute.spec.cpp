// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

#include "Misc/AutomationTest.h"
#include "Replication/Messages/ChangeClientEvent.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"

namespace UE::ConcertSyncTests::Replication::ChangeClients
{
	BEGIN_DEFINE_SPEC(FPutStateAndMuteSpec, "VirtualProduction.Concert.Replication.PutState.Mute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;

		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Sender = nullptr;
		FReplicationClient* Receiver = nullptr;

		const FGuid StreamId = FGuid::NewGuid();
	END_DEFINE_SPEC(FPutStateAndMuteSpec);

	/** This tests that muting works after a successful FConcertReplication_PutState change. */
	void FPutStateAndMuteSpec::Define()
	{
		BeforeEach([this]
		{
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Sender = &Server->ConnectClient();
			Receiver = &Server->ConnectClient();

			Sender->JoinReplication();
			Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
		});

		Describe("When client adds stream and mutes it", [this]
		{
			const auto RunTest = [this]()
			{
				IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
				FConcertReplication_PutState_Request Request;
				Request.NewStreams.Add(Sender->GetEndpointId(), { { ObjectReplicator->CreateStream(StreamId) } });
				Request.NewAuthorityState.Add(Sender->GetEndpointId()).Objects.Add({ StreamId, ObjectReplicator->TestObject });
				Request.MuteChange.ObjectsToMute.Add(ObjectReplicator->TestObject);

				bool bReceivedResponse = false;
				TFuture<FConcertReplication_PutState_Response> Future = ReplicationManager
					.PutClientState(Request)
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestTrue(TEXT("Success"), Response.IsSuccess());
						TestTrue(TEXT("No sync control"), Response.SyncControl.IsEmpty());
						return Response;
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
				return Future;
			};
			
			It("The response does not contain change for sync control", [this, RunTest]
			{
				RunTest().Next([this](FConcertReplication_PutState_Response&& Response)
				{
					TestTrue(TEXT("No sync control in response"), Response.SyncControl.IsEmpty());
				});
			});
			It("The client predicts losing sync control", [this, RunTest]
			{
				RunTest();
				
				IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
				TestEqual(TEXT("GetSyncControlledObjects().Num()"), ReplicationManager.GetSyncControlledObjects().Num(), 0);
			});
			It("The object is globally muted", [this, RunTest]
			{
				RunTest();
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestTrue(TEXT("Object is muted"), Response.ExplicitlyMutedObjects.Contains(ObjectReplicator->TestObject));
						TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num()"), Response.ExplicitlyUnmutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num()"), Response.ImplicitlyMutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num()"), Response.ImplicitlyUnmutedObjects.Num(), 0);
					});
			});
			It("The object cannot be replicated", [this, RunTest]
			{
				RunTest();

				Sender->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
				ObjectReplicator->SimulateSendObjectToReceiver(*this, { *Sender, *Server, *Receiver }, { StreamId });
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});
		});

		Describe("When a client had sync control and mutes object using put state", [this]
		{
			const auto RunTest = [this]()
			{
				IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
				ReplicationManager.ChangeStream({ .StreamsToAdd = { ObjectReplicator->CreateStream(StreamId) } });
				ReplicationManager.TakeAuthorityOver({ { ObjectReplicator->TestObject } });
				
				bool bReceivedResponse = false;
				TFuture<FConcertReplication_PutState_Response> Future = ReplicationManager
					.PutClientState({ .MuteChange = { .ObjectsToMute = { { ObjectReplicator->TestObject, {} } } } })
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestTrue(TEXT("Success"), Response.IsSuccess());
						return Response;
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
				return Future;
			};

			It("The client predicts losing sync control", [this, RunTest]
			{
				RunTest();
				
				IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
				TestEqual(TEXT("GetSyncControlledObjects().Num()"), ReplicationManager.GetSyncControlledObjects().Num(), 0);
			});
			It("The response does not contain change for sync control", [this, RunTest]
			{
				RunTest().Next([this](FConcertReplication_PutState_Response&& Response)
				{
					TestTrue(TEXT("No sync control in response"), Response.SyncControl.IsEmpty());
				});
			});
			It("The object is globally muted", [this, RunTest]
			{
				RunTest();
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestTrue(TEXT("Object is muted"), Response.ExplicitlyMutedObjects.Contains(ObjectReplicator->TestObject));
						TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num()"), Response.ExplicitlyUnmutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num()"), Response.ImplicitlyMutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num()"), Response.ImplicitlyUnmutedObjects.Num(), 0);
					});
			});
			It("The object cannot be replicated", [this, RunTest]
			{
				RunTest();
				
				Sender->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
				ObjectReplicator->SimulateSendObjectToReceiver(*this, { *Sender, *Server, *Receiver }, { StreamId });
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});
		});
		
		Describe("When client adds a stream and unmutes it", [this]
		{
			const auto RunTest = [this]()
			{
				// In order to mute an object, it must be known to the server. So we must create a dummy stream before the put request.
				IConcertClientReplicationManager& ReceiverManager = Receiver->GetClientReplicationManager();
				ReceiverManager.ChangeStream({ .StreamsToAdd = { { ObjectReplicator->CreateStream(StreamId) }} });
				ReceiverManager.MuteObjects({ ObjectReplicator->TestObject });
				
				IConcertClientReplicationManager& SenderManager = Sender->GetClientReplicationManager();
				FConcertReplication_PutState_Request Request;
				Request.NewStreams.Add(Sender->GetEndpointId(), { .Streams = { ObjectReplicator->CreateStream(StreamId) } });
				Request.NewStreams.Add(Receiver->GetEndpointId(), {});
				Request.NewAuthorityState.Add(Sender->GetEndpointId(), { .Objects = {{ StreamId, ObjectReplicator->TestObject }} });
				Request.MuteChange.ObjectsToUnmute.Add(ObjectReplicator->TestObject);
				bool bReceivedResponse = false;
				TFuture<FConcertReplication_PutState_Response> Future = SenderManager
					.PutClientState(Request)
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestTrue(TEXT("Success"), Response.IsSuccess());
						return Response;
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
				return Future;
			};
			
			It("The response contains sync control", [this, RunTest]
			{
				RunTest().Next([this](FConcertReplication_PutState_Response&& Response)
				{
					TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 1);
					const bool* SyncControlState = Response.SyncControl.NewControlStates.Find({ StreamId, ObjectReplicator->TestObject });
					TestTrue(TEXT("Has Sync Control"), SyncControlState && *SyncControlState);
				});
			});
			It("The client thinks it has sync control", [this, RunTest]
			{
				RunTest();
				
				IConcertClientReplicationManager& SenderManager = Sender->GetClientReplicationManager();
				TestEqual(TEXT("GetSyncControlledObjects().Num()"), SenderManager.GetSyncControlledObjects().Num(), 1);
				TestTrue(TEXT("GetSyncControlledObjects().Contains(TestObject)"), SenderManager.GetSyncControlledObjects().Contains({ StreamId, ObjectReplicator->TestObject }));
			});
			It("The object is not globally muted", [this, RunTest]
			{
				RunTest();
				Sender->GetClientReplicationManager()
					.QueryMuteState()
					.Next([this](FConcertReplication_QueryMuteState_Response&& Response)
					{
						TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 0);
						TestEqual(TEXT("ExplicitlyUnmutedObjects.Num()"), Response.ExplicitlyUnmutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyMutedObjects.Num()"), Response.ImplicitlyMutedObjects.Num(), 0);
						TestEqual(TEXT("ImplicitlyUnmutedObjects.Num()"), Response.ImplicitlyUnmutedObjects.Num(), 0);
					});
			});
			It("The object can be replicated", [this, RunTest]
			{
				RunTest();
				
				Sender->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
				ObjectReplicator->SimulateSendObjectToReceiver(*this, { *Sender, *Server, *Receiver }, { StreamId });
				ObjectReplicator->TestValuesWereReplicated(*this);
			});
		});

		Describe("Client receives FConcertReplication_ChangeClientEvent", [this]
		{
			BeforeEach([this]
			{
				IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
				ReplicationManager.ChangeStream({ .StreamsToAdd = {{ ObjectReplicator->CreateStream(StreamId) }} });
				ReplicationManager.TakeAuthorityOver({ ObjectReplicator->TestObject });
				ensureMsgf(ReplicationManager.GetSyncControlledObjects().Contains({ StreamId, ObjectReplicator->TestObject }), TEXT("Test not set up correctly."));
			});
			
			It("When client loses sync control due to a mute change.", [this]
			{
				int32 EventCount = 0;
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeClientEvent>(
					[this, &EventCount](const FConcertSessionContext&, const FConcertReplication_ChangeClientEvent& Event)
					{
						++EventCount;
						
						TestEqual(TEXT("Reason"), Event.Reason, EConcertReplicationChangeClientReason::PutRequest);
						TestEqual(TEXT("NewControlStates.Num()"), Event.ChangeData.SyncControlChange.NewControlStates.Num(), 1);
						const bool* NewSyncControl = Event.ChangeData.SyncControlChange.NewControlStates.Find({ StreamId, ObjectReplicator->TestObject });
						TestTrue(TEXT("Has No Sync Control"), NewSyncControl && !*NewSyncControl);
					});
				
				IConcertClientReplicationManager& ReplicationManager = Receiver->GetClientReplicationManager();
				FConcertReplication_PutState_Request Request;
				Request.MuteChange.ObjectsToMute.Add(ObjectReplicator->TestObject);
				ReplicationManager.PutClientState(Request);

				TestEqual(TEXT("EventCount"), EventCount, 1);
			});
			It("When client gains sync control due to a mute change.", [this]
			{
				int32 EventCount = 0;
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeClientEvent>(
					[this, &EventCount](const FConcertSessionContext&, const FConcertReplication_ChangeClientEvent& Event)
					{
						++EventCount;
						
						TestEqual(TEXT("Reason"), Event.Reason, EConcertReplicationChangeClientReason::PutRequest);
						TestEqual(TEXT("NewControlStates.Num()"), Event.ChangeData.SyncControlChange.NewControlStates.Num(), 1);
						const bool* NewSyncControl = Event.ChangeData.SyncControlChange.NewControlStates.Find({ StreamId, ObjectReplicator->TestObject });
						TestTrue(TEXT("Has No Sync Control"), NewSyncControl && *NewSyncControl);
					});
				
				IConcertClientReplicationManager& ReplicationManager = Receiver->GetClientReplicationManager();
				ReplicationManager.MuteObjects({ ObjectReplicator->TestObject });
				FConcertReplication_PutState_Request Request;
				Request.MuteChange.ObjectsToUnmute.Add(ObjectReplicator->TestObject);
				ReplicationManager.PutClientState(Request);

				TestEqual(TEXT("EventCount"), EventCount, 1);
			});
		});
		
		It("When client is not affected by a mute change, it does not receive FConcertReplication_ChangeClientEvent.", [this]
		{
			Receiver->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeClientEvent>(
				[this](const FConcertSessionContext&, const FConcertReplication_ChangeClientEvent& Event)
				{
					AddError(TEXT("Event was not expected"));
				});
			
			FObjectTestReplicator BarReplicator;
			IConcertClientReplicationManager& ReceiverManager = Receiver->GetClientReplicationManager();
			ReceiverManager.ChangeStream({ .StreamsToAdd = { BarReplicator.CreateStream(StreamId) } });
			ReceiverManager.TakeAuthorityOver({ BarReplicator.TestObject });
			
			IConcertClientReplicationManager& SenderManager = Sender->GetClientReplicationManager();
			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Sender->GetEndpointId(), {{ ObjectReplicator->CreateStream(StreamId) }});
			Request.MuteChange.ObjectsToMute.Add(ObjectReplicator->TestObject);
			SenderManager.PutClientState(Request);
		});
	}
}
