// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::ChangeClients
{
	BEGIN_DEFINE_SPEC(FPutStateAndReplicateSpec, "VirtualProduction.Concert.Replication.PutState.Replicate", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;

		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* OldSender = nullptr;
		FReplicationClient* NewSender = nullptr;
		FReplicationClient* Receiver = nullptr;

		const FGuid StreamId = FGuid::NewGuid();
		FConcertReplicationStream StreamData;
	
		FObjectReplicationContext MakeReplicationContext(FReplicationClient& Sender) const { return { Sender, *Server, *Receiver }; }

		FConcertReplication_PutState_Request BuildRequest()
		{
			FConcertReplication_PutState_Request Request;

			FConcertReplicationStreamArray OldSenderStreamContent { };
			Request.NewStreams.Add(OldSender->GetEndpointId(), OldSenderStreamContent);

			FConcertReplicationStreamArray NewSenderStreamContent { { StreamData } };
			Request.NewStreams.Add(NewSender->GetEndpointId(), NewSenderStreamContent);

			const FConcertObjectInStreamID ObjectId = { StreamId, ObjectReplicator->TestObject };
			Request.NewAuthorityState.Add(NewSender->GetEndpointId(), { { ObjectId } });
			
			return Request;
		}
	
		void BuildReplicationCases()
		{
			It("OldSender has empty state", [this]
			{
				const IConcertClientReplicationManager& ReplicationManager = OldSender->GetClientReplicationManager();
				TestEqual(TEXT("RegisteredStreams.Num()"), ReplicationManager.GetRegisteredStreams().Num(), 0);
				TestEqual(TEXT("ClientOwnedObjects.Num()"), ReplicationManager.GetClientOwnedObjects().Num(), 0);
				TestEqual(TEXT("SyncControl.Num()"), ReplicationManager.GetSyncControlledObjects().Num(), 0);
			});
			
			It("NewSender has correct state", [this]
			{
				const IConcertClientReplicationManager& ReplicationManager = NewSender->GetClientReplicationManager();
				
				const TArray<FConcertReplicationStream> Streams = ReplicationManager.GetRegisteredStreams();
				if (Streams.Num() != 1)
				{
					AddError(TEXT("Expected 1 stream"));
					return;
				}
				TestEqual(TEXT("Stream content"), Streams[0], StreamData);

				const TMap<FSoftObjectPath, TSet<FGuid>> AuthorityObjects = ReplicationManager.GetClientOwnedObjects();
				const TSet<FGuid>* AuthorityStreams = AuthorityObjects.Find(ObjectReplicator->TestObject);
				if (!AuthorityStreams)
				{
					AddError(TEXT("Expected authority"));
					return;
				}
				TestEqual(TEXT("AuthorityStreams.Num()"), AuthorityStreams->Num(), 1);
				TestTrue(TEXT("AuthorityStreams.Contains(StreamId)"), AuthorityStreams->Contains(StreamId));
				TestEqual(TEXT("AuthorityObjects.Num()"), AuthorityObjects.Num(), 1);

				const TSet<FConcertObjectInStreamID> SyncControl = ReplicationManager.GetSyncControlledObjects();
				TestEqual(TEXT("SyncControl.Num()"), SyncControl.Num(), 1);
				TestTrue(TEXT("SyncControl.Contains(TestObject)"), SyncControl.Contains({ StreamId, ObjectReplicator->TestObject }));
			});

			It("OldSender cannot replicate", [this]
			{
				ObjectReplicator->SimulateSendObjectToReceiver(*this, { *OldSender, *Server, *Receiver }, { StreamId });
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});
			
			It("NewSender can replicate", [this]
			{
				ObjectReplicator->SimulateSendObjectToReceiver(*this, { *NewSender, *Server, *Receiver }, { StreamId });
				ObjectReplicator->TestValuesWereReplicated(*this);
			});
		}
	END_DEFINE_SPEC(FPutStateAndReplicateSpec);

	/**
	 * This tests that replication works after a successful FConcertReplication_PutState change.
	 * It transfers authority from OldSender to NewSender and ensures that replication to Receiver still works.
	 */
	void FPutStateAndReplicateSpec::Define()
	{
		BeforeEach([this]
		{
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			OldSender = &Server->ConnectClient();
			NewSender = &Server->ConnectClient();
			Receiver = &Server->ConnectClient();

			StreamData = ObjectReplicator->CreateStream(StreamId);
			OldSender->JoinReplication(ObjectReplicator->CreateSenderArgs(StreamId));
			NewSender->JoinReplication();
			Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });
			
			OldSender->GetClientReplicationManager().TakeAuthorityOver({ ObjectReplicator->TestObject });
			OldSender->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
			NewSender->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);

			// Just double-checking that we set up the test correctly...
			ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeReplicationContext(*OldSender), { StreamId });
			ObjectReplicator->TestValuesWereReplicated(*this);
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
		});

		Describe("When Receiver transfers from OldSender to NewSender", [this]
		{
			BeforeEach([this]
			{
				Receiver->GetClientReplicationManager().PutClientState(BuildRequest());
			});
			
			BuildReplicationCases();
		});
		
		Describe("When NewSender transfers from OldSender to NewSender", [this]
		{
			BeforeEach([this]
			{
				bool bReceivedResponse = false;
				NewSender->GetClientReplicationManager()
					.PutClientState(BuildRequest())
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestTrue(TEXT("IsSuccess"), Response.IsSuccess());
						
						const bool* pbHasSyncControl = Response.SyncControl.NewControlStates.Find({ StreamId, ObjectReplicator->TestObject });
						TestTrue(TEXT("Has Sync Control"), pbHasSyncControl && *pbHasSyncControl);
						TestEqual(TEXT("SyncControl.Num()"), Response.SyncControl.NewControlStates.Num(), 1);
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
			});
			
			BuildReplicationCases();
		});
	}
}
