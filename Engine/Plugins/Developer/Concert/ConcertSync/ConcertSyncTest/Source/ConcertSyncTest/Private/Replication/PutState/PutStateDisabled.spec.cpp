// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/Messages/PutState.h"

namespace UE::ConcertSyncTests::Replication::ChangeClients
{
	BEGIN_DEFINE_SPEC(FPutStateDisabledSpec, "VirtualProduction.Concert.Replication.PutState", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		const FGuid ClientId { 0, 0, 0, 1 };
	
		void TestRequestHasErrorCode(EConcertReplicationPutStateResponseCode ExpectedErrorCode, EConcertSyncSessionFlags Flags, FConcertReplication_PutState_Request Request = {})
		{
			TUniquePtr<FReplicationServer> Server = MakeUnique<FReplicationServer>(*this, Flags);
			FReplicationClient& Client = Server->ConnectClient();
			check(Client.GetEndpointId() == ClientId);
			Client.JoinReplication();
			
			bool bReceivedResponse = false;
			Client.GetClientReplicationManager()
				.PutClientState(Request)
				.Next([this, &bReceivedResponse, ExpectedErrorCode](FConcertReplication_PutState_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("Error code"), Response.ResponseCode, ExpectedErrorCode);
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
		}
	END_DEFINE_SPEC(FPutStateDisabledSpec);

	/**
	 * This tests that FConcertReplication_PutState_Request works as intended depending on the session's EConcertSyncSessionFlags flags.
	 * The request should be rejected when EConcertSyncSessionFlags::ShouldEnableRemoteEditing is not set.
	 */
	void FPutStateDisabledSpec::Define()
	{
		It("When EConcertSyncSessionFlags::ShouldEnableRemoteEditing is not set, then FConcertReplication_PutState_Request fails.", [this]
		{
			TestRequestHasErrorCode(
				EConcertReplicationPutStateResponseCode::FeatureDisabled,
				EConcertSyncSessionFlags::Default_MultiUserSession & ~EConcertSyncSessionFlags::ShouldEnableRemoteEditing
			);
		});
		
		Describe("When EConcertSyncSessionFlags::ShouldEnableRemoteEditing is set but EConcertSyncSessionFlags::ShouldAllowGlobalMuting is not", [this]
		{
			It("When mute request is empty, the request succeeds", [this]
			{
				TestRequestHasErrorCode(
					EConcertReplicationPutStateResponseCode::Success,
					EConcertSyncSessionFlags::Default_MultiUserSession & ~EConcertSyncSessionFlags::ShouldAllowGlobalMuting
				);
			});
			It("When mute request is non-empty, the request fails", [this]
			{
				FObjectTestReplicator ObjectReplicator;
				FConcertReplication_PutState_Request Request;
				Request.NewStreams.Add(ClientId, {{ ObjectReplicator.CreateStream() }});
				Request.MuteChange.ObjectsToMute.Add({ ObjectReplicator.TestObject });
				
				TestRequestHasErrorCode(
					EConcertReplicationPutStateResponseCode::FeatureDisabled,
					EConcertSyncSessionFlags::Default_MultiUserSession & ~EConcertSyncSessionFlags::ShouldAllowGlobalMuting,
					Request
				);
			});
		});
	}
}
