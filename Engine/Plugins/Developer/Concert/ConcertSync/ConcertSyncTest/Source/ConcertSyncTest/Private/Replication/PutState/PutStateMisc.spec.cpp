// Copyright Epic Games, Inc. All Rights Reserved.

#include "Algo/Count.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/Messages/PutState.h"
#include "Replication/Util/ClientEventRecorder.h"

namespace UE::ConcertSyncTests::Replication::ChangeClients
{
	BEGIN_DEFINE_SPEC(FPutStateMiscSpec, "VirtualProduction.Concert.Replication.PutState.Misc", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;

		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client1 = nullptr;
		FReplicationClient* Client2 = nullptr;
	
		const FGuid StreamId = FGuid::NewGuid();
	END_DEFINE_SPEC(FPutStateMiscSpec);

	/** This tests misc workflows and cases with FConcertReplication_PutState_Request that do not really fit well with other specs. */
	void FPutStateMiscSpec::Define()
	{
		BeforeEach([this]
		{
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Client1 = &Server->ConnectClient();
			Client2 = &Server->ConnectClient();

			Client1->JoinReplication();
			Client2->JoinReplication();
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
		});

		// This was a bug where the server would not remove the stream from the client.
		Describe("When a put request removes client state", [this]
		{
			BeforeEach([this]
			{
				IConcertClientReplicationManager& ReplicationManager = Client1->GetClientReplicationManager();
				ReplicationManager.ChangeStream({ .StreamsToAdd = { ObjectReplicator->CreateStream(StreamId) } });
				ReplicationManager.PutClientState({ .NewStreams = { { Client1->GetEndpointId(), {} } } });
			});
			
			It("The client state can create a new stream", [this]
			{
				bool bReceivedResponse = false;
				Client1->GetClientReplicationManager()
					.ChangeStream({ .StreamsToAdd = { ObjectReplicator->CreateStream(StreamId) } })
					.Next([this, &bReceivedResponse](FConcertReplication_ChangeStream_Response&& Response)
					{
						bReceivedResponse = true;
						TestTrue(TEXT("Success"), Response.IsSuccess());
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
			});

			It("The stream has been fully deleted from the session", [this]
			{
				bool bReceivedResponse = false;
				Client1->GetClientReplicationManager()
					.QueryClientInfo({ .ClientEndpointIds = { Client1->GetEndpointId() } })
					.Next([this, &bReceivedResponse](FConcertReplication_QueryReplicationInfo_Response&& Response)
					{
						bReceivedResponse = true;

						const FConcertQueriedClientInfo* ClientInfo = Response.ClientInfo.Find(Client1->GetEndpointId());
						TestTrue(TEXT("No streams"), ClientInfo && ClientInfo->Streams.IsEmpty());
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
			});
		});

		It("OnPreRemoteEditApplied and OnPostRemoteEditApplied are triggered in right order", [this]
		{
			const FClientEventRecorder EventRecorder(Client1->GetClientReplicationManager());
			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Client1->GetEndpointId(), {{ ObjectReplicator->CreateStream(StreamId) }});
			Request.NewAuthorityState.Add(Client1->GetEndpointId()).Objects.Add({ StreamId, ObjectReplicator->TestObject });
			Client2->GetClientReplicationManager().PutClientState(Request);

			const TArray<EEventType>& ActualOrder = EventRecorder.GetEventOrder();
			if (ActualOrder.Num() < 8) // There is Pre & Post for the 4 events: Stream, Authority, SyncControl, RemoteEdit
			{
				AddError(TEXT("No all events included"));
				return;
			}
			
			TestEqual(TEXT("PreRemoteEditApplied comes first"), ActualOrder[0], EEventType::PreRemoteEditApplied);
			TestEqual(TEXT("PreRemoteEditApplied comes last"), ActualOrder[ActualOrder.Num() - 1], EEventType::PostRemoteEditApplied);

			const int32 NumPreRemoteEditAppliedBroadcasts = Algo::Count(ActualOrder, EEventType::PreRemoteEditApplied);
			const int32 NumPostRemoteEditAppliedBroadcasts = Algo::Count(ActualOrder, EEventType::PostRemoteEditApplied);
			TestEqual(TEXT("PreRemoteEditApplied appears once"), NumPreRemoteEditAppliedBroadcasts, 1);
			TestEqual(TEXT("PostRemoteEditApplied appears once"), NumPostRemoteEditAppliedBroadcasts, 1);
		});
		It("OnPreRemoteEditApplied and OnPostRemoteEditApplied have right reason", [this]
		{
			int32 EventCount = 0;
			IConcertClientReplicationManager& ReplicationManager = Client1->GetClientReplicationManager();
			const auto HandleEvent = [this, &EventCount](const ConcertSyncClient::Replication::FRemoteEditEvent& Event)
			{
				++EventCount;
				TestEqual(TEXT("Reason"), Event.Reason, EConcertReplicationChangeClientReason::PutRequest);
			};
			ReplicationManager.OnPreRemoteEditApplied().AddLambda(HandleEvent);
			ReplicationManager.OnPostRemoteEditApplied().AddLambda(HandleEvent);
			
			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Client1->GetEndpointId(), {{ ObjectReplicator->CreateStream(StreamId) }});
			Request.NewAuthorityState.Add(Client1->GetEndpointId()).Objects.Add({ StreamId, ObjectReplicator->TestObject });
			Client2->GetClientReplicationManager().PutClientState(Request);

			TestEqual(TEXT("EventCount"), EventCount, 2);
		});
	}
}
