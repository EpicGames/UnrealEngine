// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::ChangeClients
{
	BEGIN_DEFINE_SPEC(FPutStateRejectionSpec, "VirtualProduction.Concert.Replication.PutState.Rejection", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;

		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client1 = nullptr;
		FReplicationClient* Client2 = nullptr;
	
		const FGuid StreamId = FGuid::NewGuid();
		FConcertReplicationStream StreamData;
		FConcertReplicationStream StreamData_FloatOnly;
		FConcertReplicationStream StreamData_VectorOnly;

		const FConcertReplicatedObjectInfo& GetReplicatedObjectInfo() const { return StreamData.BaseDescription.ReplicationMap.ReplicatedObjects[ObjectReplicator->TestObject]; }
		const TSet<FConcertPropertyChain>& GetReplicatedProperties() const { return GetReplicatedObjectInfo().PropertySelection.ReplicatedProperties; }

		void TestClientHasConflictWith(const FConcertReplication_PutState_Response& Response, const FGuid& ClientThatHasConflict, const FGuid& ClientConflictingWith)
		{
			const FConcertAuthorityConflictArray* AuthorityErrors = Response.AuthorityChangeConflicts.Find(ClientThatHasConflict);
			if (!AuthorityErrors)
			{
				AddError(TEXT("No error generated"));
				return;
			}

			if (AuthorityErrors->Conflicts.Num() != 1)
			{
				AddError(TEXT("Expected exactly 1 error"));
				return;
			}

			const FConcertAuthorityConflict& Conflict = AuthorityErrors->Conflicts[0];
			const FConcertObjectInStreamID AttemptedObjectId { StreamId, ObjectReplicator->TestObject };
			TestEqual(TEXT("AttemptedObject"), Conflict.AttemptedObject, AttemptedObjectId);
			TestEqual(TEXT("ConflictingObject"), Conflict.ConflictingObject, { { AttemptedObjectId }, ClientConflictingWith });
		}
	END_DEFINE_SPEC(FPutStateRejectionSpec);

	/** This tests cases of where FConcertReplication_PutState_Request should be rejected or accepted. */
	void FPutStateRejectionSpec::Define()
	{
		BeforeEach([this]
		{
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Client1 = &Server->ConnectClient();
			Client2 = &Server->ConnectClient();
			Client1->JoinReplication();
			Client2->JoinReplication();
			
			StreamData = ObjectReplicator->CreateStream(StreamId);
			StreamData_FloatOnly = ObjectReplicator->CreateStreamWithProperties(StreamId, EPropertyTypeFlags::Float);
			StreamData_VectorOnly = ObjectReplicator->CreateStreamWithProperties(StreamId, EPropertyTypeFlags::Vector);
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
		});

		Describe("When an endpoint ID is invalid", [this]
		{
			const auto MakeRequest = [this](EConcertReplicationPutStateFlags Flags)
			{
				const FGuid& ClientId = Client1->GetEndpointId();
				FConcertReplication_PutState_Request Request;
				Request.Flags = Flags;
				Request.NewStreams.Add(Client1->GetEndpointId(), {{ StreamData }});
				Request.NewStreams.Add(FGuid(~ClientId.A, 0, 0, 0), {{ StreamData }});
				return Request;
			};
			
			It("EConcertReplicationChangeClientsFlags::SkipDisconnectedClients is set, the request succeeds", [this, MakeRequest]
			{
				IConcertClientReplicationManager& ReplicationManager = Client1->GetClientReplicationManager();
				
				bool bReceivedResponse = false;
				ReplicationManager
					.PutClientState(MakeRequest(EConcertReplicationPutStateFlags::SkipDisconnectedClients))
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestTrue(TEXT("Success"), Response.IsSuccess());
					});
				
				TestTrue(TEXT("Received response"), bReceivedResponse);
				TestEqual(TEXT("Stream data was applied"), ReplicationManager.GetRegisteredStreams(), { StreamData });
			});
			It("EConcertReplicationChangeClientsFlags::SkipDisconnectedClients not set, the request fails.", [this, MakeRequest]
			{
				bool bReceivedResponse = false;
				Client1->GetClientReplicationManager()
					.PutClientState(MakeRequest(EConcertReplicationPutStateFlags::None))
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestEqual(TEXT("Response code"), Response.ResponseCode, EConcertReplicationPutStateResponseCode::ClientUnknown);
						TestEqual(TEXT("1 unknown endpoint"), Response.UnknownEndpoints.Num(), 1);
						const FGuid ExpectedEndpoint = FGuid(~Client1->GetEndpointId().A, 0, 0, 0);
						TestTrue(TEXT("UnknwonEndpoints contains the invalid endpoint"), Response.UnknownEndpoints.Contains(ExpectedEndpoint));
					});
				TestTrue(TEXT("Received response"), bReceivedResponse);
			});
		});

		It("When creating stream with the same content for 2 clients, the request succeeds.", [this]
		{
			const FConcertReplication_ChangeStream_Request StreamChange { .StreamsToAdd = { StreamData } };
			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Client1->GetEndpointId(), {{ StreamData }});
			Request.NewStreams.Add(Client2->GetEndpointId(), {{ StreamData }});

			bool bReceivedResponse = false;
			Client1->GetClientReplicationManager()
				.PutClientState(Request)
				.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
				{
					bReceivedResponse = true;
					TestTrue(TEXT("Success"), Response.IsSuccess());
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
		});

		Describe("When client 1 has authority over object", [this]
		{
			BeforeEach([this]
			{
				const FConcertReplication_ChangeStream_Request StreamChange { .StreamsToAdd = { StreamData_FloatOnly } };
				
				Client1->GetClientReplicationManager().ChangeStream(StreamChange);
				Client1->GetClientReplicationManager().TakeAuthorityOver({ ObjectReplicator->TestObject });
				
				Client2->GetClientReplicationManager().ChangeStream(StreamChange);
			});
			
			It("When request tries to give client 2 authority, the request fails", [this]
			{
				FConcertReplication_PutState_Request Request;
				Request.NewAuthorityState.Add(Client2->GetEndpointId(), { { FConcertObjectInStreamID{ StreamId, ObjectReplicator->TestObject } } });
				
				bool bReceivedResponse = false;
				Client1->GetClientReplicationManager().PutClientState(Request)
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestEqual(TEXT("Response code"), Response.ResponseCode, EConcertReplicationPutStateResponseCode::AuthorityConflict);
						TestEqual(TEXT("AuthorityErrors.Num()"), Response.AuthorityChangeConflicts.Num(), 1);
						TestClientHasConflictWith(Response, Client2->GetEndpointId(), Client1->GetEndpointId());
					});
				TestTrue(TEXT("Received response"), bReceivedResponse);
			});

			It("When request removes authority from client 1 and gives it to client 2, the request succeeds", [this]
			{
				FConcertReplication_PutState_Request Request;
				Request.NewAuthorityState.Add(Client1->GetEndpointId(), {});
				Request.NewAuthorityState.Add(Client2->GetEndpointId(), { { FConcertObjectInStreamID{ StreamId, ObjectReplicator->TestObject } } });
				
				bool bReceivedResponse = false;
				Client1->GetClientReplicationManager()
					.PutClientState(Request)
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestTrue(TEXT("Success"), Response.IsSuccess());
					});
				TestTrue(TEXT("Received response"), bReceivedResponse);
			});

			It("When request changes client 1's stream so it no longer overlaps with client 2, and gives client 2 authority, the request succeeds", [this]
			{
				FConcertReplication_PutState_Request Request;
				Request.NewStreams.Add(Client1->GetEndpointId(), { .Streams { StreamData_VectorOnly } });
				Request.NewAuthorityState.Add(Client2->GetEndpointId(), { { FConcertObjectInStreamID{ StreamId, ObjectReplicator->TestObject } } });

				bool bReceivedResponse = false;
				Client1->GetClientReplicationManager()
					.PutClientState(Request)
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						TestTrue(TEXT("Success"), Response.IsSuccess());
					});
				TestTrue(TEXT("Received response"), bReceivedResponse);
			});
		});

		It("When request tries to give two clients overlapping authority, the request fails", [this]
		{
			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Client1->GetEndpointId(), { .Streams { StreamData } });
			Request.NewStreams.Add(Client2->GetEndpointId(), { .Streams { StreamData } });
			Request.NewAuthorityState.Add(Client1->GetEndpointId(), { { FConcertObjectInStreamID{ StreamId, ObjectReplicator->TestObject } } });
			Request.NewAuthorityState.Add(Client2->GetEndpointId(), { { FConcertObjectInStreamID{ StreamId, ObjectReplicator->TestObject } } });

			bool bReceivedResponse = false;
			Client1->GetClientReplicationManager()
				.PutClientState(Request)
				.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("Response code"), Response.ResponseCode, EConcertReplicationPutStateResponseCode::AuthorityConflict);
					TestEqual(TEXT("AuthorityErrors.Num()"), Response.AuthorityChangeConflicts.Num(), 2);
					TestClientHasConflictWith(Response, Client1->GetEndpointId(), Client2->GetEndpointId());
					TestClientHasConflictWith(Response, Client2->GetEndpointId(), Client1->GetEndpointId());
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
		});

		It("When request tries to mute object that is unknown, the request fails", [this]
		{
			FConcertReplication_PutState_Request Request;
			Request.MuteChange.ObjectsToMute.Add(ObjectReplicator->TestObject);
			
			bool bReceivedResponse = false;
			Client1->GetClientReplicationManager()
				.PutClientState(Request)
				.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("Response code"), Response.ResponseCode, EConcertReplicationPutStateResponseCode::MuteError);
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
		});
		It("When request tries to mute object that will become unknown, the request fails", [this]
		{
			IConcertClientReplicationManager& ReplicationManager = Client1->GetClientReplicationManager();
			ReplicationManager.ChangeStream({ .StreamsToAdd = { StreamData } });
			
			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Client1->GetEndpointId());
			Request.MuteChange.ObjectsToMute.Add(ObjectReplicator->TestObject);
			bool bReceivedResponse = false;
			ReplicationManager.PutClientState(Request)
				.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("Response code"), Response.ResponseCode, EConcertReplicationPutStateResponseCode::MuteError);
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
		});
		It("When request tries to mute an object that will become known, the request succeeds", [this]
		{
			IConcertClientReplicationManager& ReplicationManager = Client1->GetClientReplicationManager();
			
			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Client1->GetEndpointId(), {{ StreamData }});
			Request.MuteChange.ObjectsToMute.Add(ObjectReplicator->TestObject);
			bool bReceivedResponse = false;
			ReplicationManager.PutClientState(Request)
				.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
				{
					bReceivedResponse = true;
					TestTrue(TEXT("Success"), Response.IsSuccess());
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
		});

		It("When request puts an empty stream, the request fails", [this]
		{
			IConcertClientReplicationManager& ReplicationManager = Client1->GetClientReplicationManager();

			FConcertReplication_PutState_Request Request;
			Request.NewStreams.Add(Client1->GetEndpointId(), { .Streams= { FConcertReplicationStream{} } });
			bool bReceivedResponse = false;
			ReplicationManager
				.PutClientState(Request)
				.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("Response code"), Response.ResponseCode, EConcertReplicationPutStateResponseCode::StreamError);
				});
			TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
		});
	}
}
