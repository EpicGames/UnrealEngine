// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"

#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::ChangeClients
{
	BEGIN_DEFINE_SPEC(FPutStateSyncControlSpec, "VirtualProduction.Concert.Replication.PutState.SyncControl", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;

		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Sender = nullptr;
		FReplicationClient* Receiver = nullptr;

		const FGuid StreamId = FGuid::NewGuid();
		FConcertReplicationStream StreamData;
	END_DEFINE_SPEC(FPutStateSyncControlSpec);

	/**
	 * This tests that sync control is correct after a FConcertReplication_PutState change.
	 */
	void FPutStateSyncControlSpec::Define()
	{
		BeforeEach([this]
		{
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Sender = &Server->ConnectClient();
			Receiver = &Server->ConnectClient();

			StreamData = ObjectReplicator->CreateStream(StreamId);
			Sender->JoinReplication(ObjectReplicator->CreateSenderArgs(StreamId));
			Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });
			
			IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
			ReplicationManager.TakeAuthorityOver({ ObjectReplicator->TestObject });
			
			// Make sure that we've set up the test correctly: Sender should now have sync control.
			const bool bHasSyncControl = ReplicationManager.GetSyncControlledObjects().Contains({ StreamId, ObjectReplicator->TestObject });
			TestTrue(TEXT("bHasSyncControl"), bHasSyncControl);
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
		});

		Describe("When client puts empty state", [this]
		{
			const auto RunTest = [this]()
			{
				IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
			
				const FConcertReplication_PutState_Request Request { .NewStreams = { { Sender->GetEndpointId(), {} }} };
				bool bReceivedResponse = false;
				TFuture<FConcertReplication_PutState_Response> Future = ReplicationManager
					.PutClientState(Request)
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						// The change should not show up in the response because the client should predict that it does not have any more sync control.
						return Response;
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
				return Future;
			};
		
			It("Response contains no sync control change", [this, RunTest]()
			{
				RunTest()
					.Next([this](FConcertReplication_PutState_Response&& Response)
					{
						TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 0);
					});
			});
			It("Local client cache thinks it has no sync control", [this, RunTest]
			{
				RunTest();
				TestEqual(TEXT("No sync control"), Sender->GetClientReplicationManager().GetSyncControlledObjects().Num(), 0);
			});
		});

		Describe("When client's put request retains sync control", [this]
		{
			const auto RunTest = [this](bool bTakeAuthority)
			{
				IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
			
				FConcertReplication_PutState_Request Request;
				Request.NewStreams.Add(Sender->GetEndpointId()).Streams.Add(ObjectReplicator->CreateStream(StreamId));
				if (bTakeAuthority)
				{
					Request.NewAuthorityState.Add(Sender->GetEndpointId()).Objects.Add({ StreamId, ObjectReplicator->TestObject });
				}
				bool bReceivedResponse = false;
				TFuture<FConcertReplication_PutState_Response> Future = ReplicationManager
					.PutClientState(Request)
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						return Response;
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
				return Future;
			};
		
			It("Response contains no sync control change (specify authority: true)", [this, RunTest]()
			{
				RunTest(true)
					.Next([this](FConcertReplication_PutState_Response&& Response)
					{
						// The client is expected tp predict that it no longer has sync control.
						TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 0);
					});
			});
			It("Response contains no sync control change (specify authority: false)", [this, RunTest]()
			{
				RunTest(true)
					.Next([this](FConcertReplication_PutState_Response&& Response)
					{
						// The client is expected tp predict that it no longer has sync control.
						TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 0);
					});
			});
			
			It("Local client cache thinks it has sync control (specify authority: true)", [this, RunTest]
			{
				RunTest(true);
				
				// The client should have predicted that it no longer has sync control.
				TestEqual(TEXT("GetSyncControlledObjects().Num()"), Sender->GetClientReplicationManager().GetSyncControlledObjects().Num(), 1);
				TestTrue(TEXT("Has Sync Control"), Sender->GetClientReplicationManager().GetSyncControlledObjects().Contains({ StreamId, ObjectReplicator->TestObject }));
			});
			It("Local client cache thinks it has sync control (specify authority: false)", [this, RunTest]
			{
				RunTest(false);
				
				// The client should have predicted that it no longer has sync control.
				TestEqual(TEXT("GetSyncControlledObjects().Num()"), Sender->GetClientReplicationManager().GetSyncControlledObjects().Num(), 1);
				TestTrue(TEXT("Has Sync Control"), Sender->GetClientReplicationManager().GetSyncControlledObjects().Contains({ StreamId, ObjectReplicator->TestObject }));
			});
		});

		Describe("When client has sync control and removes object", [this]
		{
			const auto RunTest = [this]()
			{
				IConcertClientReplicationManager& ReplicationManager = Sender->GetClientReplicationManager();
				const FConcertReplication_PutState_Request Request { .NewStreams = {  { Sender->GetEndpointId(), {}} } };
				bool bReceivedResponse = false;
				TFuture<FConcertReplication_PutState_Response> Future = ReplicationManager
					.PutClientState(Request)
					.Next([this, &bReceivedResponse](FConcertReplication_PutState_Response&& Response)
					{
						bReceivedResponse = true;
						return Response;
					});
				TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
				return Future;
			};

			It("Response contains no sync control change", [this, RunTest]
			{
				RunTest()
					.Next([this](FConcertReplication_PutState_Response&& Response)
					{
						// The client is expected tp predict that it no longer has sync control.
						TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 0);
					});
			});
			It("Local client cache does not think it has sync control", [this, RunTest]
			{
				RunTest();
				
				// The client should have predicted that it no longer has sync control.
				TestEqual(TEXT("GetSyncControlledObjects().Num()"), Sender->GetClientReplicationManager().GetSyncControlledObjects().Num(), 0);
			});
		});
	}
}
