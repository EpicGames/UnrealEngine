// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/SendReceiveObjectTestBase.h"

#include "Replication/AuthorityConflictSharedUtils.h"
#include "Replication/Formats/FullObjectFormat.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Replication/Misc/IReplicationGroundTruth.h"
#include "TestReflectionObject.h"
#include "Util/ConcertClientReplicationBridgeMock.h"
#include "Util/SendReceiveGenericStreamTestBase.h"

#include "Components/StaticMeshComponent.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

namespace UE::ConcertSyncTests::Replication::Authority
{
	/** Data sent from a client that does not have authority over objects is rejected. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FRejectUnauthorativeClientTest, FSendReceiveObjectTestBase, "VirtualProduction.Concert.Replication.Authority.RejectUnauthorativeClient", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FRejectUnauthorativeClientTest::RunTest(const FString& Parameters)
	{
		// 1. Init
		SetUpClientAndServer();
		
		// 2.1 Generate replication data
		ConcertSyncCore::FFullObjectFormat SerializeObject;
		const TOptional<FConcertSessionSerializedPayload> Payload = SerializeObject.CreateReplicationEvent(*TestObject, [](const FArchiveSerializedPropertyChain* Chain, const FProperty& Property)
		{
			return true;
		});
		if (!Payload)
		{
			AddError(TEXT("Faild to create payload"));
			return false;
		}
		
		FConcertReplication_ObjectReplicationEvent ObjectReplicationEvent;
		ObjectReplicationEvent.ReplicatedObject = TestObject;
		ObjectReplicationEvent.SerializedPayload = *Payload;
		FConcertReplication_StreamReplicationEvent ReplicationEvent;
		ReplicationEvent.StreamId = SenderStreamId;
		ReplicationEvent.ReplicatedObjects.Add(ObjectReplicationEvent);
		FConcertReplication_BatchReplicationEvent ReplicationBatchEvent;
		ReplicationBatchEvent.Streams.Add(ReplicationEvent);
		
		// 2.2 Prepare for sending data without authority > Reject
		bool bHasServerReceivedData = false;
		auto OnServerReceive = [this, &bHasServerReceivedData](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event) mutable
		{
			if (bHasServerReceivedData)
			{
				AddError(TEXT("Server was expected to receive data exactly once!"));
			}
			bHasServerReceivedData = true;
		};
		auto OnClientReceive = [this](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event) mutable
		{
			AddError(TEXT("Server sent data from non-authorative client to receiving client!"));
		};
		ServerSession->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(OnServerReceive);
		Client_Receiver->ClientSessionMock->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(OnClientReceive);
		const FGuid ServerSessionId { 0, 0, 0, 0};
		
		// 3. Test that server processed the data and rejected it.
		IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("Concert.Replication.LogReceivedObjects"));
		if (ConsoleVariable)
		{
			// Hacky way of making sure that the server REALLY rejected the change...
			// In the future we should add a callback that can be registered via the server instance.
			AddExpectedError(TEXT("Rejected 1 object change"));
			bool bValue;
			ConsoleVariable->GetValue(bValue);
			ConsoleVariable->Set(true);
			
			// Send data without authority > Reject
			Client_Sender->ClientSessionMock->SendCustomEvent(ReplicationBatchEvent, { ServerSessionId }, EConcertMessageFlags::ReliableOrdered);
			TickServer();
			TestTrue(TEXT("Server received replication event from non-authorative client"), bHasServerReceivedData);
			
			ConsoleVariable->Set(bValue);
		}
		else
		{
			Client_Sender->ClientSessionMock->SendCustomEvent(ReplicationBatchEvent, { ServerSessionId }, EConcertMessageFlags::ReliableOrdered);
			TickServer();
			TestTrue(TEXT("Server received replication event from non-authorative client"), bHasServerReceivedData);
		}
		
		return true;
	}

	/** Makes sure only one client can take authority over the same object properties at the same time and that other clients continue being able to take authority after authority has been released.  */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAuthorityIsMutallyExclusiveTest, FSendReceiveGenericTestBase, "VirtualProduction.Concert.Replication.Authority.AuthorityIsMutallyExclusive", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FAuthorityIsMutallyExclusiveTest::RunTest(const FString& Parameters)
	{
		// 1. Init
		const UObject* UnusedTestObject = GetDefault<UTestReflectionObject>();
		const FSoftObjectPath TestObjectPath { UnusedTestObject };
		
		const FGuid SenderStreamId = FGuid::NewGuid();
		const FGuid ReceiverStreamId = SenderStreamId;
		SenderArgs = CreateHandshakeArgsFrom(*UnusedTestObject, SenderStreamId);
		ReceiverArgs = SenderArgs;
		
		SetUpClientAndServer();

		// 2.1 Senders takes authority over an object...
		bool bSenderReceivedResponse = false;
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObjectPath })
			.Next([this, &bSenderReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bSenderReceivedResponse = true;
				TestEqual(TEXT("No rejection taking authority"), Response.RejectedObjects.Num(), 0);
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TickClient(Client_Sender);
		TickServer();
		TestTrue(TEXT("Sender received response to take authority"), bSenderReceivedResponse);

		// 2.2 ... so does receiver, who fails taking authority over the same object
		bool bReceiverReceivedResponse = false;
		ClientReplicationManager_Receiver->TakeAuthorityOver({ TestObjectPath })
			.Next([this, &TestObjectPath, &ReceiverStreamId, &bReceiverReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bReceiverReceivedResponse = true;
				TestEqual(TEXT("Rejected because Sender already has authority"), Response.RejectedObjects.Num(), 1);
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				
				if (const FConcertStreamArray* RejectedStreams = Response.RejectedObjects.Find(TestObjectPath))
				{
					TestEqual(TEXT("1 stream rejected"), RejectedStreams->StreamIds.Num(), 1);
					TestTrue(TEXT("Rejected streams contains stream registered by receiver client"), RejectedStreams->StreamIds.Contains(ReceiverStreamId));
				}
				else
				{
					AddError(TEXT("Expected rejected stream array for TestObjectPath"));
				}
			});
		TickClient(Client_Receiver);
		TickServer();
		TestTrue(TEXT("Receiver received rejection response"), bReceiverReceivedResponse);

		// 2.3 ... then the sender lets go of authority ...
		bSenderReceivedResponse = false;
		ClientReplicationManager_Sender->ReleaseAuthorityOf({ TestObjectPath })
			.Next([this, &bSenderReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bSenderReceivedResponse = true;
				TestEqual(TEXT("No rejection releasing object"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TickClient(Client_Sender);
		TickServer();
		TestTrue(TEXT("Sender received response releasing object"), bSenderReceivedResponse);

		// 2.4 ... and the receiver can take authority of the object
		bReceiverReceivedResponse = false;
		ClientReplicationManager_Receiver->TakeAuthorityOver({ TestObjectPath })
			.Next([this, &bReceiverReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bReceiverReceivedResponse = true;
				TestEqual(TEXT("No rejection because the object should not be released"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TickClient(Client_Receiver);
		TickServer();
		TestTrue(TEXT("Receiver received response taking authority"), bReceiverReceivedResponse);
		
		return true;
	}
	
	/** Clients cannot take authority over objects that they did not send */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCannotTakeAuthorityOverUnregisteredObjectsTest, FSendReceiveGenericTestBase, "VirtualProduction.Concert.Replication.Authority.CannotTakeAuthorityOverUnregisteredObjects", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FCannotTakeAuthorityOverUnregisteredObjectsTest::RunTest(const FString& Parameters)
	{
		// FSendReceiveGenericTestBase is used here because it registers no sending streams by default
		
		// 1. Init
		SetUpClientAndServer();

		// 2. Attempt to take authority over object that was not in handshake
		bool bReceivedResponse = false;
		const FSoftObjectPath SomePath(GetDefault<UTestReflectionObject>());
		const FGuid StreamId = FGuid::NewGuid();
		FConcertReplication_ChangeAuthority_Request Request;
		Request.TakeAuthority.Add(SomePath, FConcertStreamArray{ .StreamIds = { StreamId } });
		// Detail: Cannot use IConcertClientReplicationManager::TakeAuthority util here because it builds the request based on what streams were registered
		ClientReplicationManager_Sender->RequestAuthorityChange(Request)
			.Next([this, &bReceivedResponse, &SomePath, &StreamId](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bReceivedResponse = true;
				TestTrue(TEXT("Cannot take authority over unregistered object"), Response.RejectedObjects.Contains(SomePath));
				TestEqual(TEXT("Rejected exactly 1 object"), Response.RejectedObjects.Num(), 1);
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);

				if (const FConcertStreamArray* RejectedStreams = Response.RejectedObjects.Find(SomePath))
				{
					TestEqual(TEXT("1 stream rejected"), RejectedStreams->StreamIds.Num(), 1);
					TestTrue(TEXT("Rejected streams contains stream registered by sender client"), RejectedStreams->StreamIds.Contains(StreamId));
				}
				else
				{
					AddError(TEXT("Expected rejected stream array for SomePath"));
				}
			});

		// 3. Authority request was rejected
		SimulateSenderToReceiver();
		TestTrue(TEXT("Authority request was answered"), bReceivedResponse);
		
		return true;
	}

	/** When a client leaves the replication session, the client releases all of the authority as well.  */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FClientLeavingReplicationSessionLosesAuthority, FSendReceiveGenericTestBase, "VirtualProduction.Concert.Replication.Authority.ClientLeavingReplicationSessionLosesAuthority", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FClientLeavingReplicationSessionLosesAuthority::RunTest(const FString& Parameters)
	{
		// 1. Init
		const UObject* UnusedTestObject = GetDefault<UTestReflectionObject>();
		const FSoftObjectPath TestObjectPath { UnusedTestObject };
		SenderArgs = CreateHandshakeArgsFrom(*UnusedTestObject);
		ReceiverArgs = SenderArgs;
		
		SetUpClientAndServer();

		// 2.1 Senders takes authority over an object...
		bool bSenderReceivedResponse = false;
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObjectPath })
			.Next([this, &bSenderReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bSenderReceivedResponse = true;
				TestEqual(TEXT("No rejection taking authority (sender)"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TickClient(Client_Sender);
		TickServer();
		TestTrue(TEXT("Sender received response to take authority"), bSenderReceivedResponse);

		// 2.2 ... then leaves
		ClientReplicationManager_Sender->LeaveReplicationSession();
		TickClient(Client_Sender);
		TickServer();
		
		// 2.3 which means the receiver is allowed to take authority
		bool bReceiverReceivedResponse = false;
		ClientReplicationManager_Receiver->TakeAuthorityOver({ TestObjectPath })
			.Next([this, &bReceiverReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bReceiverReceivedResponse = true;
				TestEqual(TEXT("No rejection taking authority (receiver)"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TickClient(Client_Receiver);
		TickServer();
		TestTrue(TEXT("Receiver received response to take authority"), bReceiverReceivedResponse);
		
		return true;
	}

	/** When a client leaves the replication session, the client releases all of the authority as well.  */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FClientLeavingConcertSessionLosesAuthority, FSendReceiveGenericTestBase, "VirtualProduction.Concert.Replication.Authority.ClientLeavingConcertSessionLosesAuthority", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FClientLeavingConcertSessionLosesAuthority::RunTest(const FString& Parameters)
	{
		// 1. Init
		const UObject* UnusedTestObject = GetDefault<UTestReflectionObject>();
		const FSoftObjectPath TestObjectPath { UnusedTestObject };
		SenderArgs = CreateHandshakeArgsFrom(*UnusedTestObject);
		ReceiverArgs = SenderArgs;
		
		SetUpClientAndServer();

		// 2.1 Senders takes authority over an object...
		bool bSenderReceivedResponse = false;
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObjectPath })
			.Next([this, &bSenderReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bSenderReceivedResponse = true;
				TestEqual(TEXT("No rejection taking authority (sender)"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TickClient(Client_Sender);
		TickServer();
		TestTrue(TEXT("Sender received response to take authority"), bSenderReceivedResponse);

		// 2.2 ... then leaves
		Client_Sender->ClientSessionMock->Disconnect();
		TickClient(Client_Sender);
		TickServer();
		
		// 2.3 which means the receiver is allowed to take authority
		bool bReceiverReceivedResponse = false;
		ClientReplicationManager_Receiver->TakeAuthorityOver({ TestObjectPath })
			.Next([this, &bReceiverReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bReceiverReceivedResponse = true;
				TestEqual(TEXT("No rejection taking authority (receiver)"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TickClient(Client_Receiver);
		TickServer();
		TestTrue(TEXT("Receiver received response to take authority"), bReceiverReceivedResponse);
		
		return true;
	}
	
	/**
	 * Tests that the client's local bridge is updated if and only if changing authority:
	 * - Taking authority calls IConcertClientReplicationBridge::PushTrackedObjects
	 * - Releasing authority calls IConcertClientReplicationBridge::ReleaseTrackedObjects
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FChangingAuthorityUpdatesClientBridge, FSendReceiveObjectTestBase, "VirtualProduction.Concert.Replication.Authority.ChangingAuthorityUpdatesClientBridge", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FChangingAuthorityUpdatesClientBridge::RunTest(const FString& Parameters)
	{
		// 1. Init
		SetUpClientAndServer();

		// 2. Send authority
		TestEqual(TEXT("No tracked objects"), BridgeMock_Sender->TrackedObjects.Num(), 0);
		bool bSenderReceivedTakeResponse = false;
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObject })
			.Next([this, &bSenderReceivedTakeResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bSenderReceivedTakeResponse = true;
				TestEqual(TEXT("No rejection taking authority"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TestTrue(TEXT("Sender received response taking authority"), bSenderReceivedTakeResponse);

		// 3. Test: Added TestObject to tracked objects
		TestEqual(TEXT("Tracked exactly 1 object"), BridgeMock_Sender->TrackedObjects.Num(), 1);
		TestTrue(TEXT("Tracked TestObject"), BridgeMock_Sender->TrackedObjects.Contains(TestObject));

		// 4. Release authority
		bool bSenderReceivedReleaseResponse = false;
		ClientReplicationManager_Sender->ReleaseAuthorityOf({ TestObject })
			.Next([this, &bSenderReceivedReleaseResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bSenderReceivedReleaseResponse = true;
				TestEqual(TEXT("No rejection releasing authority"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TestTrue(TEXT("Sender received response releasing authority"), bSenderReceivedReleaseResponse);
		
		// 4. Test: Removed TestObject from tracked objects
		TestEqual(TEXT("Removed tracked object"), BridgeMock_Sender->TrackedObjects.Num(), 0);

		return true;
	}

	/**
	 * Tests that client updates its local cache of the server state when RequestAuthorityChange times out.
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FChangingAuthorityTimeoutRetainsServerState, FSendReceiveObjectTestBase, "VirtualProduction.Concert.Replication.Authority.ChangeAuthorityTimeoutRetainsServerState", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FChangingAuthorityTimeoutRetainsServerState::RunTest(const FString& Parameters)
	{
		// 1. Init
		SetUpClientAndServer();
		ServerSession->SetTestFlags(EServerSessionTestingFlags::AllowRequestTimeouts);
		
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObject });
		ServerSession->UnregisterCustomRequestHandler<FConcertReplication_ChangeAuthority_Request>();
		ClientReplicationManager_Sender->ReleaseAuthorityOf({ TestObject });

		TestTrue(TEXT("Timed out ReleaseAuthorityOf reverted local server prediction"), ClientReplicationManager_Sender->GetClientOwnedObjects().Contains(TestObject));
		return true;
	}

	namespace ConflictEnumerationComponent
	{
		constexpr FGuid RequestingClientId { 0, 0, 0, 0};
		constexpr FGuid ExistingClientId { 1, 0, 0, 0};
		
		class FTestGroundTruth : public ConcertSyncCore::Replication::IReplicationGroundTruth
		{
		public:
			
			FConcertBaseStreamInfo RequestingClientStream;
			FConcertBaseStreamInfo ExistingClientStream;
			bool bRequestingClientHasAuthority;
			bool bExistingClientHasAuthority;

			FTestGroundTruth(const FConcertBaseStreamInfo& RequestingClientStream, const FConcertBaseStreamInfo& ExistingClientStream, bool bRequestingClientHasAuthority, bool bExistingClientHasAuthority)
				: RequestingClientStream(RequestingClientStream)
				, ExistingClientStream(ExistingClientStream)
				, bRequestingClientHasAuthority(bRequestingClientHasAuthority)
				, bExistingClientHasAuthority(bExistingClientHasAuthority)
			{}

			virtual void ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)> Callback) const override
			{
				if (ClientEndpointId == RequestingClientId)
				{
					Callback(RequestingClientStream.Identifier, RequestingClientStream.ReplicationMap);
				}
				if (ClientEndpointId == ExistingClientId)
				{
					Callback(ExistingClientStream.Identifier, ExistingClientStream.ReplicationMap);
				}
			}
			
			virtual void ForEachClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const override
			{
				if (bRequestingClientHasAuthority && Callback(RequestingClientId) == EBreakBehavior::Break)
				{
					return;
				}
				if (bExistingClientHasAuthority)
				{
					Callback(ExistingClientId);
				}
			}
			
			virtual bool HasAuthority(const FGuid& ClientId, const FGuid& StreamId, const FSoftObjectPath& ObjectPath) const override
			{
				const bool bRequestor = ClientId == RequestingClientId && RequestingClientStream.Identifier == StreamId && bRequestingClientHasAuthority;
				const bool bExisting = ClientId == ExistingClientId && ExistingClientStream.Identifier == StreamId && bExistingClientHasAuthority;
				return bRequestor || bExisting;
			}
		};
		
		FConcertBaseStreamInfo MakeStream(const FGuid& StreamId, const FSoftObjectPath ObjectPath, TSet<FConcertPropertyChain> Properties)
		{
			FConcertObjectReplicationMap ExistingReplicationMap;
			const FConcertPropertySelection Selection{ Properties};
			ExistingReplicationMap.ReplicatedObjects.Add(ObjectPath, { {}, Selection });
			const FConcertBaseStreamInfo ExistingClientStream { StreamId, ExistingReplicationMap };
			return ExistingClientStream;
		};
	}

	/** Tests UE::ConcertSyncCore::Replication::AuthorityConflictUtils::EnumerateAuthorityConflicts. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FConflictEnumerationComponenTests, FSendReceiveObjectTestBase, "VirtualProduction.Concert.Replication.Authority.ConflictEnumerationComponenTests", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FConflictEnumerationComponenTests::RunTest(const FString& Parameters)
	{
		using namespace ConflictEnumerationComponent;
		using namespace ConcertSyncCore::Replication::AuthorityConflictUtils;
		constexpr FGuid TestStreamId { 0, 0, 0, 1};
		const FSoftObjectPath ObjectPath(TEXT("/Game/World.World:PersistentLevel.StaticMeshActor0"));
		
		FConcertPropertyChain Float = *FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Float") });
		FConcertPropertyChain Vector = *FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector") });
		FConcertPropertyChain VectorX = *FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector"), TEXT("X") });
		const FConcertBaseStreamInfo ClientStream_Empty = MakeStream(TestStreamId, ObjectPath, {});
		const FConcertBaseStreamInfo ClientStream_FloatOnly = MakeStream(TestStreamId, ObjectPath, { Float });
		const FConcertBaseStreamInfo ClientStream_VectorOnly = MakeStream(TestStreamId, ObjectPath, { Vector, VectorX });

		// Requestor tries to stream Float, then Vector
		// Existing is replicating Float
		{
			const FTestGroundTruth GroundTruth { ClientStream_Empty, ClientStream_FloatOnly, false, true };

			int32 NumCalls = 0;
			const EAuthorityConflict FloatConflict = EnumerateAuthorityConflicts(RequestingClientId, ObjectPath, { Float }, GroundTruth,
				[this, &TestStreamId, &Float, &NumCalls](const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertyChain& ConflictingProperty)
				{
					++NumCalls;
					TestEqual(TEXT("Float conflict > Client"), ClientId, ExistingClientId);
					TestEqual(TEXT("Float conflict > Stream"), StreamId, TestStreamId);
					TestEqual(TEXT("Float conflict > Property"), Float, ConflictingProperty);
					return EBreakBehavior::Continue;
				});
			TestTrue(TEXT("1 Float overlaps"), FloatConflict == EAuthorityConflict::Conflict);
			TestEqual(TEXT("1 Float > Exactly 1 conflicting property"), NumCalls, 1);
			
			const EAuthorityConflict VectorConflict = EnumerateAuthorityConflicts(RequestingClientId, ObjectPath, { Vector, VectorX }, GroundTruth,
				[this](const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertyChain& ConflictingProperty)
				{
					AddError(TEXT("No conflict expected"));
					return EBreakBehavior::Continue;
				});
			TestTrue(TEXT("1 Vector does not overlap"), VectorConflict == EAuthorityConflict::Allowed);
		}

		// Requestor tries to stream Float, then Vector
		// Existing is not replicating but has Float registered
		{
			const FTestGroundTruth GroundTruth { ClientStream_Empty, ClientStream_FloatOnly, false, false };
			const EAuthorityConflict FloatConflict = EnumerateAuthorityConflicts(RequestingClientId, ObjectPath, { Float }, GroundTruth,
				[this](const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertyChain& ConflictingProperty)
				{
					AddError(TEXT("No conflict expected"));
					return EBreakBehavior::Continue;
				});
			TestTrue(TEXT("2 Float does not overlap"), FloatConflict == EAuthorityConflict::Allowed);
			const EAuthorityConflict VectorConflict = EnumerateAuthorityConflicts(RequestingClientId, ObjectPath, { Vector, VectorX }, GroundTruth,
				[this](const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertyChain& ConflictingProperty)
				{
					AddError(TEXT("No conflict expected"));
					return EBreakBehavior::Continue;
				});
			TestTrue(TEXT("2 Vector overlaps"), VectorConflict == EAuthorityConflict::Allowed);
		}

		// Requestor replicating Float and requests Vector
		// Existing replicating Vector
		{
			const FTestGroundTruth GroundTruth { ClientStream_FloatOnly, ClientStream_VectorOnly, true, true };
			int32 NumCalls = 0;
			const EAuthorityConflict FloatConflict = EnumerateAuthorityConflicts(RequestingClientId, ObjectPath, { Float, Vector, VectorX }, GroundTruth,
				[this, &TestStreamId, &Vector, &VectorX, &NumCalls](const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertyChain& ConflictingProperty)
				{
					++NumCalls;
					TestEqual(TEXT("Float conflict > Client"), ClientId, ExistingClientId);
					TestEqual(TEXT("Float conflict > Stream"), StreamId, TestStreamId);
					TestTrue(TEXT("Correct roperty"), Vector == ConflictingProperty || VectorX == ConflictingProperty);
					return EBreakBehavior::Continue;
				});
			TestTrue(TEXT("3 Float overlaps"), FloatConflict == EAuthorityConflict::Conflict);
			TestEqual(TEXT("3 Float > Exactly 2 conflicting properties"), NumCalls, 2);
		}
		
		return true;
	}
}
