// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/ClientServerCommunicationTest.h"

#include "Replication/Data/ReplicationStream.h"
#include "Replication/IConcertClientReplicationBridge.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/IConcertServerReplicationManager.h"
#include "Replication/Messages/Handshake.h"
#include "ReplicationTestInterface.h"
#include "Util/ConcertClientReplicationBridgeMock.h"

#include "Components/StaticMeshComponent.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Util/Mocks/ReplicationWorkspaceEmptyMock.h"

namespace UE::ConcertSyncTests::Replication::Handshake
{
	/**
	 * Tests the handshake for joining and leaving a replication session.
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FJoinHandshakeTest, FConcertClientServerCommunicationTest, "VirtualProduction.Concert.Replication.Handshake", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FJoinHandshakeTest::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncClient::TestInterface;
		using namespace ConcertSyncServer::TestInterface;
		using namespace ConcertSyncClient::Replication;
		using namespace ConcertSyncServer::Replication;
		const FSoftObjectPath PathToSomeActorComponent(TEXT("/Game/Map.Map:PersistentLevel.StaticMeshActor0.StaticMeshComponent0"));
		const FConcertPropertyChain ForcedLodModelProperty = *FConcertPropertyChain::CreateFromPath(*UStaticMeshComponent::StaticClass(), { TEXT("ForcedLodModel") });
		const FConcertPropertyChain MinLODProperty = *FConcertPropertyChain::CreateFromPath(*UStaticMeshComponent::StaticClass(), { TEXT("MinLOD") });

		// 1. Init
		
		// Server
		InitServer();
		const TSharedPtr<IConcertServerSession>& ServerSession = GetServerSessionMock();
		FReplicationWorkspaceEmptyMock WorkspaceEmptyMock;
		const TSharedRef<IConcertServerReplicationManager> ServerReplicationManager = CreateServerReplicationManager(ServerSession.ToSharedRef(), WorkspaceEmptyMock);
		
		// Client
		FClientInfo& Client = ConnectClient();
		const TSharedRef<IConcertClientReplicationBridge> BridgeMock = MakeShared<FConcertClientReplicationBridgeMock>();
		const TSharedPtr<IConcertClientSession>& ClientSession = Client.ClientSessionMock;
		const TSharedRef<IConcertClientReplicationManager> ClientReplicationManager_Primary = CreateClientReplicationManager(ClientSession.ToSharedRef(), BridgeMock.Get());
		FClientInfo& Client_Secondary = ConnectClient();
		const TSharedRef<IConcertClientReplicationBridge> BridgeMock_Secondary = MakeShared<FConcertClientReplicationBridgeMock>();
		const TSharedPtr<IConcertClientSession>& ClientSession_Secondary = Client_Secondary.ClientSessionMock;
		const TSharedRef<IConcertClientReplicationManager> ClientReplicationManager_Secondary = CreateClientReplicationManager(ClientSession_Secondary.ToSharedRef(), BridgeMock_Secondary.Get());

		// Stream 
		FConcertReplicationStream StreamDescription;
		FConcertReplicatedObjectInfo StaticMeshComponentInfo;
		StaticMeshComponentInfo.ClassPath = UStaticMeshComponent::StaticClass();
		StaticMeshComponentInfo.PropertySelection.ReplicatedProperties.Add(ForcedLodModelProperty);
		StreamDescription.BaseDescription.ReplicationMap.ReplicatedObjects.Add(PathToSomeActorComponent, StaticMeshComponentInfo);

		FConcertReplicationStream DuplicateStreamDescription = StreamDescription;
		DuplicateStreamDescription.BaseDescription.Identifier = FGuid::NewGuid();
		
		FConcertReplicationStream InvalidClassStreamDescription = StreamDescription;
		FConcertReplicatedObjectInfo MissingClassInfo = StaticMeshComponentInfo;
		MissingClassInfo.ClassPath.Reset();
		InvalidClassStreamDescription.BaseDescription.ReplicationMap.ReplicatedObjects.Add(PathToSomeActorComponent, MissingClassInfo);

		
		// 2. Run

		// 2.1 Invalid configurations
		// These should cases never happen if using the editor tools.
		// However malicious users or those with custom C++ logic can send whatever they want.
		
		// 2.1.1 Duplicate stream identifier
		ClientReplicationManager_Primary->JoinReplicationSession({ { StreamDescription, StreamDescription } })
			.Next([&](const FJoinReplicatedSessionResult& Result)
			{
				TestTrue(TEXT("Cannot contain stream ID twice"), Result.ErrorCode == EJoinReplicationErrorCode::DuplicateStreamId);
			});
		// 2.1.2 Missing class path
		ClientReplicationManager_Primary->JoinReplicationSession({ { InvalidClassStreamDescription } })
			.Next([&](const FJoinReplicatedSessionResult& Result)
			{
				TestTrue(TEXT("Cannot contain null classes"), Result.ErrorCode == EJoinReplicationErrorCode::InvalidClass);
			});


		// 2.2 Real workflow cases
		// 2.2.1 Valid join
		ClientReplicationManager_Primary->JoinReplicationSession({ { StreamDescription } })
			.Next([&](const FJoinReplicatedSessionResult& Result)
			{
				TestTrue(TEXT("Join with valid args"), Result.ErrorCode == EJoinReplicationErrorCode::Success);
			});
		// 2.2.2 No joining twice
		AddExpectedError(TEXT("JoinReplicationSession requested while already in a session"), EAutomationExpectedErrorFlags::Contains); // Not pretty, but otherwise this test fails due to logged warning
		ClientReplicationManager_Primary->JoinReplicationSession({ { StreamDescription } })
			.Next([&](const FJoinReplicatedSessionResult& Result)
			{
				TestTrue(TEXT("Cannot join twice"), Result.ErrorCode == EJoinReplicationErrorCode::AlreadyInSession);
			});
		// 2.2.3 Rejoining
		ClientReplicationManager_Primary->LeaveReplicationSession();
		ClientReplicationManager_Primary->JoinReplicationSession({ { StreamDescription } })
			.Next([&](const FJoinReplicatedSessionResult& Result)
			{
				TestTrue(TEXT("Re-join session"), Result.ErrorCode == EJoinReplicationErrorCode::Success);
			});
		
		// 2.2.4 Two clients may join with the same stream identifier and properties. 
		// For context: UX-wise it should be as easy as possible for clients to join.
		// This is why we allow clients to join with overlapping properties and stream IDs, which used to be rejected by the handshake.
		// Now, they must take authority over objects before sending. See authority tests.
		FConcertReplicationStream DuplicateProperties = StreamDescription;
		ClientReplicationManager_Secondary->JoinReplicationSession({ { StreamDescription } })
			.Next([&](const FJoinReplicatedSessionResult& Result)
			{
				TestTrue(TEXT("2nd client can overlap stream identifier and properties of 1st client"), Result.ErrorCode == EJoinReplicationErrorCode::Success);
			});
		
		return true;
	}
}
