// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/SendReceiveObjectTestBase.h"

#include "Replication/Data/ReplicationStream.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/IConcertServerReplicationManager.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Replication/Messages/Handshake.h"
#include "Replication/PropertyChainUtils.h"
#include "Replication/ReplicationTestInterface.h"
#include "TestReflectionObject.h"
#include "Util/ConcertClientReplicationBridgeMock.h"
#include "Util/ClientServerCommunicationTest.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::ConcertSyncTests::Replication::SendReceiveFlow
{
	/** Sets up clients, server, and makes the sender take authority over the test object. */
	static void SharedSetupSet(FSendReceiveObjectTestBase& Test)
	{
		Test.SetUpClientAndServer();
		Test.ClientReplicationManager_Sender->TakeAuthorityOver({ Test.TestObject })
			.Next([&Test](FConcertReplication_ChangeAuthority_Response&& Response)
			{
				if (!Response.RejectedObjects.IsEmpty())
				{
					Test.AddError(TEXT("Failed to take authority"));
				}
			});
	}
	
	/** Send values from sender > server > receiver and validate it arrived. */
	static void SharedRunSendReceiveTest(FSendReceiveObjectTestBase& Test,
		EPropertyTestFlags PropertyTestFlags = EPropertyTestFlags::All,
		// Explicitly sets all values before & after sending so the caller can test whether the other properties were changed
		EPropertyTestFlags PropertySendFlags = EPropertyTestFlags::All
		)
	{
		bool bHasServerReceivedData = false;
		bool bHasClientReceivedData = false;
		auto OnServerReceive = [&Test, &bHasServerReceivedData](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event) mutable
		{
			if (bHasServerReceivedData)
			{
				Test.AddError(TEXT("Server was expected to receive data exactly once!"));
			}
			bHasServerReceivedData = true;
		};
		auto OnClientReceive = [&Test, &bHasClientReceivedData](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event) mutable
		{
			if (bHasClientReceivedData)
			{
				Test.AddError(TEXT("Client 2 was expected to receive data exactly once!"));
			}
			bHasClientReceivedData = true;
		};
		Test.SimulateSendObjectToReceiver(OnServerReceive, OnClientReceive,
			
			PropertySendFlags
			);

		// 3. Test
		Test.TestTrue(TEXT("Server received replication event"), bHasServerReceivedData);
		Test.TestTrue(TEXT("Client 2 received replication event"), bHasClientReceivedData);
		Test.TestEqualTestValues(*Test.TestObject, PropertyTestFlags);
	}

	/** Changes the sending stream so it sends only the Float property. */
	static void ChangeStreamToSendingVectorOnly(FSendReceiveObjectTestBase& Test)
	{
		FConcertPropertySelection VectorOnlySelection;
		const TOptional<FConcertPropertyChain> VectorPropertyChain = FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector") });
		VectorOnlySelection.ReplicatedProperties.Add(*VectorPropertyChain);
		
		FConcertReplication_ChangeStream_Request Request;
		Request.ObjectsToPut.Add(FConcertObjectInStreamID{ Test.SenderStreamId, Test.TestObject }, FConcertReplication_ChangeStream_PutObject{ VectorOnlySelection });
		bool bReceivedChangeStreamResponse = false;
		Test.ClientReplicationManager_Sender->ChangeStream(Request)
			.Next([&Test, &bReceivedChangeStreamResponse](FConcertReplication_ChangeStream_Response&& Response)
			{
				bReceivedChangeStreamResponse = true;
				Test.TestTrue(TEXT("Changed Stream"), Response.IsSuccess());
				Test.TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		Test.TestTrue(TEXT("Received change stream response"), bReceivedChangeStreamResponse);
	}
	
	/** Tests replicating data from sender client > server > receiver client. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FSendReceiveFlowTests, FSendReceiveObjectTestBase, "VirtualProduction.Concert.Replication.SendReceive.SendValuesOnSingleStream", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FSendReceiveFlowTests::RunTest(const FString& Parameters)
	{
		SharedSetupSet(*this);
		SharedRunSendReceiveTest(*this);
		return true;
	}
	
	/** Tests replicating data from sender client > server > receiver client. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FSendCDOValuesOnSingleStream, FSendReceiveObjectTestBase, "VirtualProduction.Concert.Replication.SendReceive.SendCDOValuesOnSingleStream", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FSendCDOValuesOnSingleStream::RunTest(const FString& Parameters)
	{
		SharedSetupSet(*this);
		SharedRunSendReceiveTest(*this, EPropertyTestFlags::All | EPropertyTestFlags::SendCDOValues, EPropertyTestFlags::All | EPropertyTestFlags::SendCDOValues);
		return true;
	}

	/** Test which still sends TestObject but does so with two separate streams: one for the float and the other for the vector property. */
	class FSplitSendReceiveObjectTest : public FSendReceiveObjectTestBase
	{
	public:
		FSplitSendReceiveObjectTest(const FString& InName, const bool bInComplexTask)
        	: FSendReceiveObjectTestBase(InName, bInComplexTask)
        {}

	protected:

		FGuid FloatStreamId = FGuid::NewGuid();
		FGuid VectorStreamId = FGuid::NewGuid();
		
		//~ Begin FSendReceiveTestBase Interface
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateSenderArgs() override
		{
			ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderJoinArgs;
			const UClass& TestClass = *TestObject->GetClass();

			FConcertReplicatedObjectInfo FloatProperties {&TestClass};
			FloatProperties.PropertySelection.ReplicatedProperties.Add(*FConcertPropertyChain::CreateFromPath(TestClass, { GET_MEMBER_NAME_CHECKED(UTestReflectionObject, Float) }));
			FConcertReplicatedObjectInfo VectorProperties { &TestClass };
			VectorProperties.PropertySelection.ReplicatedProperties.Add(*FConcertPropertyChain::CreateFromPath(TestClass, { GET_MEMBER_NAME_CHECKED(UTestReflectionObject, Vector) }));

			FConcertReplicationStream FloatStream;
			FloatStream.BaseDescription.Identifier = FloatStreamId;
			FloatStream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(TestObject, FloatProperties);
			// Use realtime replication, otherwise we'll have to pass time for the frequency system
			FloatStream.BaseDescription.FrequencySettings.Defaults.ReplicationMode = EConcertObjectReplicationMode::Realtime;
			
			FConcertReplicationStream VectorStream;
			VectorStream.BaseDescription.Identifier = VectorStreamId;
			VectorStream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(TestObject, VectorProperties);
			// Use realtime replication, otherwise we'll have to pass time for the frequency system
			VectorStream.BaseDescription.FrequencySettings.Defaults.ReplicationMode = EConcertObjectReplicationMode::Realtime;
			
			SenderJoinArgs.Streams.Add(FloatStream);
			SenderJoinArgs.Streams.Add(VectorStream);
			return SenderJoinArgs;
		}

		virtual TSet<FGuid> GetSenderStreamIds() const override { return { FloatStreamId, VectorStreamId }; }
		//~ End FSendReceiveTestBase Interface
	};

	/** Test which still sends TestObject but does so with two separate streams: one for the float and the other for the vector property. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FSendReceiveMultiStreamSameObjectTests, FSplitSendReceiveObjectTest, "VirtualProduction.Concert.Replication.SendReceive.MultipleStreamsForSameObject", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FSendReceiveMultiStreamSameObjectTests::RunTest(const FString& Parameters)
	{
		SharedSetupSet(*this);
		SharedRunSendReceiveTest(*this);
		return true;
	}

	/** Tests that changing a stream while replication is in progress actually changes the properties being replicated. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FUpdateStreamWhileSendingUpdatesSentProperties, FSendReceiveObjectTestBase, "VirtualProduction.Concert.Replication.SendReceive.UpdateStreamWhileSendingUpdatesSentProperties", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FUpdateStreamWhileSendingUpdatesSentProperties::RunTest(const FString& Parameters)
	{
		// 1. Send all properties with the stream from handshake
		SharedSetupSet(*this);
		SharedRunSendReceiveTest(*this);

		// 2. Change stream to only send floats and test only the float is sent.
		ChangeStreamToSendingVectorOnly(*this);
		SharedRunSendReceiveTest(*this, EPropertyTestFlags::Vector);
		// Test that only the vector was written
		TestEqualDifferentValues(*TestObject, EPropertyTestFlags::Float);
		return true;
	}
}
