// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncSessionFlags.h"

#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

class IConcertClientSession;
class IConcertServerSession;
class IConcertClientReplicationBridge;
class IConcertClientReplicationManager;
class FConcertSyncSessionDatabase;
struct FConcertSessionClientInfo;
struct FGuid;

namespace UE::ConcertSyncServer::Replication
{
	class IConcertServerReplicationManager;
	class IReplicationWorkspace;
}

namespace UE::ConcertSyncClient::TestInterface
{
	extern CONCERTSYNCCLIENT_API TSharedRef<IConcertClientReplicationManager> CreateClientReplicationManager(
		TSharedRef<IConcertClientSession> InLiveSession,
		IConcertClientReplicationBridge& InBridge UE_LIFETIMEBOUND,
		EConcertSyncSessionFlags SessionFlags = EConcertSyncSessionFlags::Default_MultiUserSession
		);

	extern CONCERTSYNCCLIENT_API TSharedRef<IConcertClientReplicationBridge> CreateClientReplicationBridge();
}

namespace UE::ConcertSyncServer::TestInterface
{
	extern CONCERTSYNCSERVER_API TSharedRef<Replication::IConcertServerReplicationManager> CreateServerReplicationManager(
		TSharedRef<IConcertServerSession> InLiveSession,
		Replication::IReplicationWorkspace& InWorkspace UE_LIFETIMEBOUND,
		EConcertSyncSessionFlags InSessionFlags = EConcertSyncSessionFlags::Default_MultiUserSession
		);

	extern CONCERTSYNCSERVER_API TSharedRef<Replication::IReplicationWorkspace> CreateReplicationWorkspace(
		FConcertSyncSessionDatabase& Database,
		TFunction<TOptional<FConcertSessionClientInfo>(const FGuid& EndpointId)> FindSessionClient,
		TFunction<bool(const FGuid& ClientId)> ShouldIgnoreClientActivityOnRestore
		);
}