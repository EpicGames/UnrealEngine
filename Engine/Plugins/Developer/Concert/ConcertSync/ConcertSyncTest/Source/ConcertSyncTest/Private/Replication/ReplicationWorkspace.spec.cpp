// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationTestInterface.h"
#include "Misc/AutomationTest.h"
#include "Replication/SyncControlState.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Replication/Messages/SyncControl.h"
#include "Util/ConcertClientReplicationBridgeMock.h"
#include "Util/ScopedSessionDatabase.h"
#include "Util/Spec/ObjectTestReplicator.h"
#include "Util/Spec/ReplicationClient.h"
#include "Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	BEGIN_DEFINE_SPEC(FReplicationWorkspaceSpec, "VirtualProduction.Concert.Replication.ReplicationWorkspace", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FScopedSessionDatabase> SessionDatabase;
		TSharedPtr<ConcertSyncServer::Replication::IReplicationWorkspace> ReplicationWorkspace;
	
		TUniquePtr<FObjectTestReplicator> Object;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;

		FGuid StreamId = FGuid::NewGuid();
		FGuid SecondaryStreamId = FGuid::NewGuid();

		FReplicationClient& ConnectClient(const FString& DisplayName, const FString& DeviceName)
		{
			FReplicationClient& CreatedClient = Server->ConnectClient({ .DeviceName = DeviceName, .DisplayName = DisplayName });
			SessionDatabase->SetEndpoint(CreatedClient.GetEndpointId(), { CreatedClient.GetClientInfo() });
			return CreatedClient;
		}

		FConcertSyncReplicationPayload_LeaveReplication MakeExpectedData() const { return { .Streams = { Object->CreateStream(StreamId) } }; }
		FConcertSessionClientInfo GetClientSessionInfo() const { return { Client->GetEndpointId(), Client->GetClientInfo() }; }
		static FString MainDisplayName() { return TEXT("PrimaryClientName"); }
		static FString MainDeviceName() { return TEXT("PrimaryMachine"); }
		static FString SecondaryDeviceName() { return TEXT("SecondaryMachine"); }
	END_DEFINE_SPEC(FReplicationWorkspaceSpec);

	/** This tests that clients FReplicationWorkspace is correctly implemented. */
	void FReplicationWorkspaceSpec::Define()
	{
		BeforeEach([this]
		{
			SessionDatabase = MakeUnique<FScopedSessionDatabase>(*this);
			const auto FindSessionClient = [this](const FGuid& EndpointId) -> TOptional<FConcertSessionClientInfo>
			{
				FConcertSessionClientInfo Info;
				const bool bFound = Server->GetServerSessionMock()->FindSessionClient(EndpointId, Info);
				return bFound ? Info : TOptional<FConcertSessionClientInfo>{};
			};
			const auto ShouldIgnoreClientActivityOnRestore = [](const FGuid& EndpointId){ return false; };
			ReplicationWorkspace = ConcertSyncServer::TestInterface::CreateReplicationWorkspace(*SessionDatabase, FindSessionClient, ShouldIgnoreClientActivityOnRestore);
			
			Object = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Client = &ConnectClient(MainDisplayName(), MainDeviceName());
			Client->JoinReplication();
		});
		AfterEach([this]
		{
			ReplicationWorkspace.Reset();
			SessionDatabase.Reset();
			Server.Reset();
		});
		
		It("ProduceClientLeaveReplicationActivity", [this]
		{
			const FConcertSyncReplicationPayload_LeaveReplication ExpectedData = MakeExpectedData();
			const FGuid ClientId = Client->GetEndpointId();
			const TOptional<int64> ActivityId = ReplicationWorkspace->ProduceClientLeaveReplicationActivity(ClientId, ExpectedData);
			if (!ActivityId)
			{
				AddError(TEXT("Failed to create activity"));
				return;
			}

			FConcertSyncReplicationActivity Activity;
			if (!SessionDatabase->GetReplicationActivity(*ActivityId, Activity))
			{
				AddError(TEXT("Activity not saved"));
				return;
			}

			FConcertSyncReplicationPayload_LeaveReplication ActualData;
			const bool bGotData = Activity.EventData.GetPayload(ActualData);
			TestTrue(TEXT("Got Data"), bGotData);
			TestEqual(TEXT("Payload"), ActualData, ExpectedData);
		});

		Describe("GetLastLeaveReplicationActivityByClient", [this]
		{
			BeforeEach([this]()
			{
				const FGuid ClientId = Client->GetEndpointId();
				ReplicationWorkspace->ProduceClientLeaveReplicationActivity(ClientId, MakeExpectedData());
			});
			
			It("Simple: Single endpoint", [this]
			{
				FConcertSyncReplicationPayload_LeaveReplication ActualData;
				const bool bGotData = ReplicationWorkspace->GetLastLeaveReplicationActivityByClient(GetClientSessionInfo(), ActualData);
				
				TestTrue(TEXT("Got Data"), bGotData);
				TestEqual(TEXT("Payload"), ActualData, MakeExpectedData());
			});
			It("When single client has changed device, use the latest data", [this]
			{
				const FReplicationClient& OtherClient = ConnectClient(MainDisplayName(), MainDeviceName());
				
				FConcertSyncReplicationPayload_LeaveReplication ActualData;
				const bool bGotData = ReplicationWorkspace->GetLastLeaveReplicationActivityByClient({ OtherClient.GetEndpointId(), OtherClient.GetClientInfo() }, ActualData);
				TestTrue(TEXT("Got Data"), bGotData);
				TestEqual(TEXT("Payload"), ActualData, MakeExpectedData());
			});
			
			It("When there are 2 clients with same display name but different device names, use the data associated with the device name", [this]
			{
				const FReplicationClient& OtherClient = ConnectClient(MainDisplayName(), SecondaryDeviceName());
				const FConcertSyncReplicationPayload_LeaveReplication OtherClientData { .Streams = { Object->CreateStream(SecondaryStreamId) } };
				const TOptional<int64> OtherClientActivityId = ReplicationWorkspace->ProduceClientLeaveReplicationActivity(OtherClient.GetEndpointId(), OtherClientData);
				
				FConcertSyncReplicationPayload_LeaveReplication ActualData;
				const bool bGotData = ReplicationWorkspace->GetLastLeaveReplicationActivityByClient(GetClientSessionInfo(), ActualData);
				TestTrue(TEXT("Got Data"), bGotData);
				TestEqual(TEXT("Payload"), ActualData, MakeExpectedData());
				// This is already handled by the Payload case but we'll make sure
				TestTrue(TEXT("StreamId"), ActualData.Streams.Num() == 1 && ActualData.Streams[0].BaseDescription.Identifier == StreamId);
			});
			It ("When there are 2 clients with the same display and device name, use the latest data", [this]
			{
				const FReplicationClient& OtherClient = ConnectClient(MainDisplayName(), MainDeviceName());
				const FConcertSyncReplicationPayload_LeaveReplication OtherClientData { .Streams = { Object->CreateStream(SecondaryStreamId) } };
				const TOptional<int64> OtherClientActivityId = ReplicationWorkspace->ProduceClientLeaveReplicationActivity(OtherClient.GetEndpointId(), OtherClientData);
				
				FConcertSyncReplicationPayload_LeaveReplication ActualData;
				const bool bGotData = ReplicationWorkspace->GetLastLeaveReplicationActivityByClient(GetClientSessionInfo(), ActualData);
				TestTrue(TEXT("Got Data"), bGotData);
				TestEqual(TEXT("Payload"), ActualData, OtherClientData);
				// This is already handled by the Payload case but we'll make sure
				TestTrue(TEXT("StreamId"), ActualData.Streams.Num() == 1 && ActualData.Streams[0].BaseDescription.Identifier == SecondaryStreamId);
			});
		});

		It("GetLeaveReplicationActivityById", [this]
		{
			const FConcertSyncReplicationPayload_LeaveReplication ExpectedData = MakeExpectedData();
			const FGuid ClientId = Client->GetEndpointId();
			const TOptional<int64> ActivityId = ReplicationWorkspace->ProduceClientLeaveReplicationActivity(ClientId, ExpectedData);
			if (!ActivityId)
			{
				AddError(TEXT("Failed to create activity"));
				return;
			}

			FConcertSyncReplicationPayload_LeaveReplication ActualData;
			const bool bGotData = ReplicationWorkspace->GetLeaveReplicationEventById(*ActivityId, ActualData);
			TestTrue(TEXT("Got Data"), bGotData);
			TestEqual(TEXT("Payload"), ActualData, ExpectedData);
		});
	}
}
