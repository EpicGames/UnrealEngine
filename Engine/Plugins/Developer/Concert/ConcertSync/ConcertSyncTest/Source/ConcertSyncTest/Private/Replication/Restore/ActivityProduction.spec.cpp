// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Mocks/ReplicationWorkspaceCallInterceptorMock.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/Messages/ReplicationActivity.h"

namespace UE::ConcertSyncTests::Replication
{
	BEGIN_DEFINE_SPEC(FLeaveReplicationActivitySpec, "VirtualProduction.Concert.Replication.RestoreContent.Activity", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		/** Detects calls into the workspace */
		TSharedPtr<FReplicationWorkspaceCallInterceptorMock> WorkspaceMock;
	
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
	
		FGuid SenderStreamId = FGuid::NewGuid();

		void ValidateLeaveActivity()
		{
			if (WorkspaceMock->LastCall_ProduceClientLeaveReplicationActivity)
			{
				const auto[EndpointId, EventData] = *WorkspaceMock->LastCall_ProduceClientLeaveReplicationActivity;
				TestEqual(TEXT("EndpointId"), EndpointId, Client->GetEndpointId());
				
				TestEqual(TEXT("Streams.Num() == 1"), EventData.Streams.Num(), 1);
				if (EventData.Streams.Num() == 1)
				{
					const FConcertBaseStreamInfo& StreamInfo = EventData.Streams[0].BaseDescription;
					TestEqual(TEXT("StreamId"), StreamInfo.Identifier, SenderStreamId);
					
					const FConcertReplicatedObjectInfo* ObjectInfo = StreamInfo.ReplicationMap.ReplicatedObjects.Find(ObjectReplicator->TestObject);
					if (ObjectInfo)
					{
						TestEqual(TEXT("ReplicatedObjects.Num() == 1"),StreamInfo.ReplicationMap.ReplicatedObjects.Num(), 1);
						TestTrue(TEXT("Class correct"), ObjectInfo->ClassPath == ObjectReplicator->TestObject->GetClass());
						
						const ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderArgs = ObjectReplicator->CreateSenderArgs(SenderStreamId);
						const FConcertPropertySelection& ExpectedProperties = SenderArgs.Streams[0].BaseDescription.ReplicationMap.ReplicatedObjects[ObjectReplicator->TestObject].PropertySelection;
						TestEqual(TEXT("Properties equal"), ObjectInfo->PropertySelection, ExpectedProperties);
					}
					else
					{
						AddError(TEXT("Replicated object not listed in stream"));
					}
				}
				
				TestEqual(TEXT("OwnedObjects.Num() == 1"), EventData.OwnedObjects.Num(), 1);
				TestTrue(TEXT("OwnedObjects contains owned object"), EventData.OwnedObjects.Contains(FConcertObjectInStreamID{ SenderStreamId, ObjectReplicator->TestObject }));
			}
			else
			{
				AddError(TEXT("No activity was produced"));
			}
		}
	END_DEFINE_SPEC(FLeaveReplicationActivitySpec);

	/** This tests that when a client leaves replication, an activity containing their registered streams and authority is produced. */
	void FLeaveReplicationActivitySpec::Define()
	{
		BeforeEach([this]
		{
			WorkspaceMock = MakeShared<FReplicationWorkspaceCallInterceptorMock>();
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession, WorkspaceMock.ToSharedRef());
			Client = &Server->ConnectClient();
			
			Client->JoinReplication(ObjectReplicator->CreateSenderArgs(SenderStreamId));
			Client->GetClientReplicationManager().TakeAuthorityOver({ ObjectReplicator->TestObject });
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
			WorkspaceMock.Reset();
		});
		
		It("When leaving replication, an activity is produced", [this]
		{
			Client->LeaveReplication();
			ValidateLeaveActivity();
		});
		It("When disconnecting from the session, an activity is produced", [this]
		{
			Client->GetClientSessionMock()->Disconnect();
			ValidateLeaveActivity();
		});
	}


	BEGIN_DEFINE_SPEC(FMuteReplicationActivitySpec, "VirtualProduction.Concert.Replication.RestoreContent.Activity", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		/** Detects calls into the workspace */
		TSharedPtr<FReplicationWorkspaceCallInterceptorMock> WorkspaceMock;
	
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
	
		FGuid SenderStreamId = FGuid::NewGuid();
	END_DEFINE_SPEC(FMuteReplicationActivitySpec);

	/** This tests that when a client leaves replication, an activity containing their registered streams and authority is produced. */
	void FMuteReplicationActivitySpec::Define()
	{
		BeforeEach([this]
		{
			WorkspaceMock = MakeShared<FReplicationWorkspaceCallInterceptorMock>();
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession, WorkspaceMock.ToSharedRef());
			Client = &Server->ConnectClient();
			
			Client->JoinReplication(ObjectReplicator->CreateSenderArgs(SenderStreamId));
			Client->GetClientReplicationManager().TakeAuthorityOver({ ObjectReplicator->TestObject });
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
			WorkspaceMock.Reset();
		});
		
		It("When muting replication, an activity is produced", [this]
		{
			bool bMuteSuccess = false;
			Client->GetClientReplicationManager()
				.MuteObjects({ ObjectReplicator->TestObject })
				.Next([&bMuteSuccess](FConcertReplication_ChangeMuteState_Response&& Response)
				{
					bMuteSuccess = Response.IsSuccess();
				});
			
			TestTrue(TEXT("Mute success"), bMuteSuccess);
			if (!WorkspaceMock->LastCall_ProduceClientMuteReplicationActivity)
			{
				AddError(TEXT("No activity produced"));
				return;
			}
			
			const auto[EndpointId, EventData] = *WorkspaceMock->LastCall_ProduceClientMuteReplicationActivity;
			const FConcertReplication_ChangeMuteState_Request& Request = EventData.Request;
			TestEqual(TEXT("EndpointId"), EndpointId, Client->GetEndpointId());
			TestEqual(TEXT("ObjectsToMute.Num()"), Request.ObjectsToMute.Num(), 1);
			TestEqual(TEXT("ObjectsToUnmute.Num()"), Request.ObjectsToUnmute.Num(), 0);
			TestTrue(TEXT("ObjectsToMute.Contains(TestObject)"), Request.ObjectsToMute.Contains(ObjectReplicator->TestObject));
		});
	}
	
	BEGIN_DEFINE_SPEC(FNoReplicationActivitiesSpec, "VirtualProduction.Concert.Replication.RestoreContent.Activity", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		/** Detects calls into the workspace */
		TSharedPtr<FReplicationWorkspaceCallInterceptorMock> WorkspaceMock;
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;
	
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
	END_DEFINE_SPEC(FNoReplicationActivitiesSpec);

	/** This tests no activities are generated when the session EConcertSyncSessionFlags::ShouldEnableReplicationActivities is not set.  */
	void FNoReplicationActivitiesSpec::Define()
	{
		BeforeEach([this]
		{
			WorkspaceMock = MakeShared<FReplicationWorkspaceCallInterceptorMock>();
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession & ~EConcertSyncSessionFlags::ShouldEnableReplicationActivities, WorkspaceMock.ToSharedRef());
			Client = &Server->ConnectClient();

			Client->JoinReplication(ObjectReplicator->CreateSenderArgs());
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
			WorkspaceMock.Reset();
		});
		
		It("When leaving replication, no activity is produced", [this]
		{
			Client->LeaveReplication();
			TestFalse(TEXT("Activity produced"), WorkspaceMock->LastCall_ProduceClientLeaveReplicationActivity.IsSet());
		});
		It("When muting, no activity is produced", [this]
		{
			bool bMuteSuccess = false;
			Client->GetClientReplicationManager()
				.MuteObjects({ ObjectReplicator->TestObject })
				.Next([&bMuteSuccess](FConcertReplication_ChangeMuteState_Response&& Response)
				{
					bMuteSuccess = Response.IsSuccess();
				});
			
			TestTrue(TEXT("Mute success"), bMuteSuccess);
			TestFalse(TEXT("Activity produced"), WorkspaceMock->LastCall_ProduceClientMuteReplicationActivity.IsSet());
		});
	}
}
