// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Mocks/ReplicationWorkspaceCallInterceptorMock.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication::RestoreContent
{
	BEGIN_DEFINE_SPEC(FRestoreContentDisabledSpec, "VirtualProduction.Concert.Replication.RestoreContent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		/** Detects calls into the workspace */
		TSharedPtr<FReplicationWorkspaceCallInterceptorMock> WorkspaceMock;
	
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
	END_DEFINE_SPEC(FRestoreContentDisabledSpec);

	/** This tests that a client's stream and authority cannot be restored when EConcertSyncSessionFlags::ShouldEnableReplicationActivities is not set. */
	void FRestoreContentDisabledSpec::Define()
	{
		BeforeEach([this]
		{
			WorkspaceMock = MakeShared<FReplicationWorkspaceCallInterceptorMock>();
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession & ~EConcertSyncSessionFlags::ShouldEnableReplicationActivities, WorkspaceMock.ToSharedRef());
			Client = &Server->ConnectClient();

			Client->JoinReplicationAsListener({});
		});
		AfterEach([this]
		{
			Server.Reset();
			WorkspaceMock.Reset();
		});
		
		It("If EConcertSyncSessionFlags::ShouldEnableReplicationActivities, then FConcertReplication_RestoreContent_Request fails.", [this]
		{
			bool bReceivedResponse = false;
			Client->GetClientReplicationManager()
				.RestoreContent()
				.Next([this, &bReceivedResponse](FConcertReplication_RestoreContent_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("Error code"), Response.ErrorCode, EConcertReplicationRestoreErrorCode::NotSupported);
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
		});
	}
}
