// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Mocks/ReplicationWorkspaceCallInterceptorMock.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"

namespace UE::ConcertSyncTests::Replication::RestoreContent
{
	BEGIN_DEFINE_SPEC(FRestoreContentPreexistingSpec, "VirtualProduction.Concert.Replication.RestoreContent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		/** Detects calls into the workspace */
		TSharedPtr<FReplicationWorkspaceCallInterceptorMock> WorkspaceMock;
		FConcertSyncReplicationPayload_LeaveReplication LeaveReplicationData;

		TUniquePtr<FObjectTestReplicator> PreexistingObject1;
		TUniquePtr<FObjectTestReplicator> RestoredObject1;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
		
		FGuid StreamId = FGuid::NewGuid();

		void RestoreAndTestStreamContent(
			const ConcertSyncClient::Replication::FJoinReplicatedSessionArgs& JoinArgs,
			TFunctionRef<void(const TArray<FConcertBaseStreamInfo>& Streams)> TestStreamContent,
			TFunctionRef<void(const TArray<FConcertAuthorityClientInfo>& Authority)> TestAuthorityContent
			)
		{
			Client->JoinReplication(JoinArgs);
			
			InsertActivityData();
			Client->GetClientReplicationManager().RestoreContent({ .Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority | EConcertReplicationRestoreContentFlags::RestoreOnTop });
			Client->GetClientReplicationManager()
				.QueryClientInfo({ .ClientEndpointIds = { Client->GetEndpointId() } })
				.Next([this, &TestStreamContent, &TestAuthorityContent](FConcertReplication_QueryReplicationInfo_Response&& Response)
				{
					const FConcertQueriedClientInfo* ClientInfo = Response.ClientInfo.Find(Client->GetEndpointId());
					if (!ClientInfo)
					{
						AddError(TEXT("Not restored correctly"));
						return;
					}

					TestStreamContent(ClientInfo->Streams);
					TestAuthorityContent(ClientInfo->Authority);
				});
		}
	
		void InsertActivityData() const
		{
			WorkspaceMock->ReturnResult_GetLastReplicationActivityByClient = {{ EConcertSyncReplicationActivityType::LeaveReplication, FConcertSyncReplicationActivity(LeaveReplicationData) }};
			WorkspaceMock->ReturnResult_GetReplicationEventById = FConcertSyncReplicationEvent(LeaveReplicationData);
		}
		FConcertReplicatedObjectInfo& GetRestoredObjectData()
		{
			return LeaveReplicationData.Streams[0].BaseDescription.ReplicationMap.ReplicatedObjects[RestoredObject1->TestObject];
		}
	END_DEFINE_SPEC(FRestoreContentPreexistingSpec);

	/** This tests that a client's stream and authority can be restored when the client has preexisting stream content and authority. */
	void FRestoreContentPreexistingSpec::Define()
	{
		BeforeEach([this]
		{
			WorkspaceMock = MakeShared<FReplicationWorkspaceCallInterceptorMock>();
			PreexistingObject1 = MakeUnique<FObjectTestReplicator>(TEXT("PreexistingObject1"));
			RestoredObject1 = MakeUnique<FObjectTestReplicator>(TEXT("RestoredObject1"));
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession, WorkspaceMock.ToSharedRef());
			Client = &Server->ConnectClient();

			LeaveReplicationData.Streams.Add(RestoredObject1->CreateStream(StreamId));
			LeaveReplicationData.OwnedObjects.Add({ StreamId, RestoredObject1->TestObject });
		});
		AfterEach([this]
		{
			WorkspaceMock.Reset();
			Server.Reset();
			PreexistingObject1.Reset();
			RestoredObject1.Reset();
			Client = nullptr;
			LeaveReplicationData = {};
		});

		It("Aggregate objects into same stream", [this]
		{
			const ConcertSyncClient::Replication::FJoinReplicatedSessionArgs JoinArgs = PreexistingObject1->CreateSenderArgs(StreamId);
			const auto ValidateStream = [this, &JoinArgs](const TArray<FConcertBaseStreamInfo>& Streams)
			{
				if (Streams.Num() != 1)
				{
					AddError(TEXT("Wrong streams"));
					return;
				}

				const FConcertBaseStreamInfo& Stream =Streams[0];
				const FConcertReplicatedObjectInfo& ExpectedRestoredContent = GetRestoredObjectData();
				const FConcertReplicatedObjectInfo& ExpectedPreexistingContent = JoinArgs.Streams[0].BaseDescription.ReplicationMap.ReplicatedObjects[PreexistingObject1->TestObject];
				const FConcertReplicatedObjectInfo* ActualRestoredContent = Stream.ReplicationMap.ReplicatedObjects.Find(RestoredObject1->TestObject);
				const FConcertReplicatedObjectInfo* ActualPreexistingContent = Stream.ReplicationMap.ReplicatedObjects.Find(PreexistingObject1->TestObject);
				TestTrue(TEXT("Preexisting"), ActualPreexistingContent && *ActualPreexistingContent == ExpectedPreexistingContent);
				TestTrue(TEXT("Restored"), ActualRestoredContent && *ActualRestoredContent == ExpectedRestoredContent);
			};
			const auto ValidateAuthority = [this](const TArray<FConcertAuthorityClientInfo>& Authority)
			{
				if (Authority.Num() != 1)
				{
					AddError(TEXT("Wrong authority"));
					return;
				}

				TestEqual(TEXT("StreamId"), Authority[0].StreamId, StreamId);
				TestEqual(TEXT("1 owned object"), Authority[0].AuthoredObjects.Num(), 1);
				TestTrue(TEXT("Restored object is owned"), Authority[0].AuthoredObjects.Contains(RestoredObject1->TestObject));
			};
			RestoreAndTestStreamContent(JoinArgs, ValidateStream, ValidateAuthority);
		});
		
		It("When restoring on top of object that already has all properties, the properties are retained", [this]
		{
			const ConcertSyncClient::Replication::FJoinReplicatedSessionArgs JoinArgs = RestoredObject1->CreateSenderArgs(StreamId);
			FConcertReplicatedObjectInfo& RestoredData = GetRestoredObjectData();
			RestoredData.PropertySelection.ReplicatedProperties = { *FConcertPropertyChain::CreateFromPath(*RestoredObject1->TestObject->GetClass(), { TEXT("Float") }) };
			
			const auto ValidateStream = [this, &JoinArgs](const TArray<FConcertBaseStreamInfo>& Streams)
			{
				if (Streams.Num() != 1)
				{
					AddError(TEXT("Wrong streams"));
					return;
				}

				const FConcertBaseStreamInfo& Stream = Streams[0];
				const FConcertReplicatedObjectInfo* ActualContent = Stream.ReplicationMap.ReplicatedObjects.Find(RestoredObject1->TestObject);
				const FConcertReplicatedObjectInfo& ExpectedContent = JoinArgs.Streams[0].BaseDescription.ReplicationMap.ReplicatedObjects[RestoredObject1->TestObject];
				TestTrue(TEXT("Preexisting"), ActualContent && *ActualContent == ExpectedContent);
			};
			const auto ValidateAuthority = [this](const TArray<FConcertAuthorityClientInfo>& Authority)
			{
				if (Authority.Num() != 1)
				{
					AddError(TEXT("Wrong authority"));
					return;
				}

				TestEqual(TEXT("StreamId"), Authority[0].StreamId, StreamId);
				TestEqual(TEXT("1 owned object"), Authority[0].AuthoredObjects.Num(), 1);
				TestTrue(TEXT("Restored object is owned"), Authority[0].AuthoredObjects.Contains(RestoredObject1->TestObject));
			};
			RestoreAndTestStreamContent(JoinArgs, ValidateStream, ValidateAuthority);
		});

		It("When restoring on top of an object, the properties are properly aggregated", [this]
		{
			const TOptional<FConcertPropertyChain> VectorProperty = FConcertPropertyChain::CreateFromPath(*RestoredObject1->TestObject->GetClass(), { TEXT("Vector") });
			const TOptional<FConcertPropertyChain> VectorXProperty = FConcertPropertyChain::CreateFromPath(*RestoredObject1->TestObject->GetClass(), { TEXT("Vector"), TEXT("X") });
			const TOptional<FConcertPropertyChain> FloatProperty = FConcertPropertyChain::CreateFromPath(*RestoredObject1->TestObject->GetClass(), { TEXT("Float") });
			if (!VectorProperty || !VectorXProperty || !FloatProperty)
			{
				AddError(TEXT("Error in FConcertPropertyChain::CreateFromPath"));
				return;
			}
			
			// RestoredObject will already have "Vector.X" to begin with ...
			ConcertSyncClient::Replication::FJoinReplicatedSessionArgs JoinArgs = RestoredObject1->CreateSenderArgs(StreamId);
			FConcertReplicatedObjectInfo& JoinData = JoinArgs.Streams[0].BaseDescription.ReplicationMap.ReplicatedObjects[RestoredObject1->TestObject];
			JoinData.PropertySelection.ReplicatedProperties = { *VectorProperty, *VectorXProperty };
			
			// ... the data to restore is "Float" ...
			FConcertReplicatedObjectInfo& RestoredContent = GetRestoredObjectData();
			RestoredContent.PropertySelection.ReplicatedProperties = { *FloatProperty };
			
			const auto ValidateStream = [this, &RestoredContent, &VectorProperty, &VectorXProperty, &FloatProperty](const TArray<FConcertBaseStreamInfo>& Streams)
			{
				if (Streams.Num() != 1)
				{
					AddError(TEXT("Wrong streams"));
					return;
				}

				const FConcertBaseStreamInfo& Stream = Streams[0];
				const FConcertReplicatedObjectInfo* ActualContent = Stream.ReplicationMap.ReplicatedObjects.Find(RestoredObject1->TestObject);
				if (!ActualContent)
				{
					AddError(TEXT("No object data"));
					return;
				}

				// ... so the object should now have all 3 properties
				const TSet<FConcertPropertyChain>& Properties = ActualContent->PropertySelection.ReplicatedProperties;
				TestEqual(TEXT("Properties.Num() == 3"), Properties.Num(), 3);
				TestTrue(TEXT("Has property: Vector"), Properties.Contains(*VectorProperty));
				TestTrue(TEXT("Has property: Vector.X"), Properties.Contains(*VectorXProperty));
				TestTrue(TEXT("Has property: Float"), Properties.Contains(*FloatProperty));
			};
			const auto ValidateAuthority = [this](const TArray<FConcertAuthorityClientInfo>& Authority)
			{
				if (Authority.Num() != 1)
				{
					AddError(TEXT("Wrong authority"));
					return;
				}

				TestEqual(TEXT("StreamId"), Authority[0].StreamId, StreamId);
				TestEqual(TEXT("1 owned object"), Authority[0].AuthoredObjects.Num(), 1);
				TestTrue(TEXT("Restored object is owned"), Authority[0].AuthoredObjects.Contains(RestoredObject1->TestObject));
			};
			RestoreAndTestStreamContent(JoinArgs, ValidateStream, ValidateAuthority);
		});
	}
}
