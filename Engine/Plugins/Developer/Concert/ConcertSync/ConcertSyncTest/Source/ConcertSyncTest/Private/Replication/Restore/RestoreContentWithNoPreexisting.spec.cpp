// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Mocks/ReplicationWorkspaceCallInterceptorMock.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"

namespace UE::ConcertSyncTests::Replication::RestoreContent
{
	enum class ERestoreTestFlags
	{
		None = 0,
		TestForAuthority = 1 << 0
	};
	ENUM_CLASS_FLAGS(ERestoreTestFlags)

	/** Test that restoring content works correctly when the restoring client has no streams nor authority pre-assigned. */
	BEGIN_DEFINE_SPEC(FRestoreContentWithNoPreExistingSpec, "VirtualProduction.Concert.Replication.RestoreContent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		/** Detects calls into the workspace */
		TSharedPtr<FReplicationWorkspaceCallInterceptorMock> WorkspaceMock;
		FConcertSyncReplicationPayload_LeaveReplication LeaveReplicationData;

		TUniquePtr<FObjectTestReplicator> RestoredObject1;
		TUniquePtr<FObjectTestReplicator> RestoredObject2;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
	
		FGuid RestoreStreamId = FGuid::NewGuid();

		void InsertActivityData() const
		{
			WorkspaceMock->ReturnResult_GetLastReplicationActivityByClient = {{ EConcertSyncReplicationActivityType::LeaveReplication, FConcertSyncReplicationActivity(LeaveReplicationData) }};
			WorkspaceMock->ReturnResult_GetReplicationEventById = FConcertSyncReplicationEvent(LeaveReplicationData);
		}

		TFuture<FConcertReplication_RestoreContent_Response> RestoreThenTestErrorCode(const FConcertReplication_RestoreContent_Request& Request, EConcertReplicationRestoreErrorCode ExpectedErrorCode)
		{
			bool bReceivedResponse = false;
			TFuture<FConcertReplication_RestoreContent_Response> Future = Client->GetClientReplicationManager()
				.RestoreContent(Request)
				.Next([this, ExpectedErrorCode, &bReceivedResponse](FConcertReplication_RestoreContent_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("Error code"), Response.ErrorCode, ExpectedErrorCode);
					return Response;
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
			return MoveTemp(Future);
		}

		void RestoreStreamAndAuthorityThenTest(const FConcertReplication_RestoreContent_Request& Request, ERestoreTestFlags Flags = ERestoreTestFlags::TestForAuthority)
		{
			// Join the client first so sync control is granted by the request.
			FReplicationClient& Receiver = Server->ConnectClient();
			Receiver.JoinReplicationAsListener({ RestoredObject1->TestObject, RestoredObject2->TestObject });

			// Do the actual restoring
			InsertActivityData();
			RestoreThenTestErrorCode(
				Request,
				EConcertReplicationRestoreErrorCode::Success
			)
			.Next([this, Flags](FConcertReplication_RestoreContent_Response&& Response)
			{
				if (EnumHasAnyFlags(Flags, ERestoreTestFlags::TestForAuthority))
				{
					TestEqual(TEXT("NewControlStates.Num() == 2"), Response.SyncControl.NewControlStates.Num(), 2);
					const bool* bNewSyncControl1 = Response.SyncControl.NewControlStates.Find({ RestoreStreamId, RestoredObject1->TestObject });
					const bool* bNewSyncControl2 = Response.SyncControl.NewControlStates.Find({ RestoreStreamId, RestoredObject2->TestObject });
					TestTrue(TEXT("Sync Control 1"), bNewSyncControl1 && *bNewSyncControl1);
					TestTrue(TEXT("Sync Control 2"), bNewSyncControl2 && *bNewSyncControl2);
				}
			});

			
			
			// Can replicate?
			if (EnumHasAnyFlags(Flags, ERestoreTestFlags::TestForAuthority))
			{
				Client->GetBridgeMock().InjectAvailableObject(*RestoredObject1->TestObject);
				RestoredObject1->SimulateSendObjectToReceiver(*this, { *Client, *Server, Receiver }, { RestoreStreamId });
				RestoredObject1->TestValuesWereReplicated(*this);
			
				// Need to hide object because RestoredObject2.SimulateSendObjectToReceiver will test that we only receive from RestoredObject2.
				Client->GetBridgeMock().HideObject(*RestoredObject1->TestObject);
				Client->GetBridgeMock().InjectAvailableObject(*RestoredObject2->TestObject);
				RestoredObject2->SimulateSendObjectToReceiver(*this, { *Client, *Server, Receiver }, { RestoreStreamId });
				RestoredObject2->TestValuesWereReplicated(*this);
			}

			
			
			// Authority and stream state correct?
			Client->GetClientReplicationManager()
				.QueryClientInfo({ { Client->GetEndpointId() } })
				.Next([this, Flags](FConcertReplication_QueryReplicationInfo_Response&& Response)
				{
					const FConcertQueriedClientInfo& ClientInfo = Response.ClientInfo[Client->GetEndpointId()];
					if (ClientInfo.Streams.Num() != 1)
					{
						AddError(TEXT("Stream not found"));
						return;
					}
					const FConcertBaseStreamInfo& Stream = ClientInfo.Streams[0];
					// This catches everything...
					TestTrue(TEXT("Stream content"), Stream == LeaveReplicationData.Streams[0].BaseDescription);
					// ... but for easier debugging we'll test some specific things now
					TestEqual(TEXT("StreamId"), Stream.Identifier, RestoreStreamId);
					TestEqual(TEXT("Frequency"), Stream.FrequencySettings, LeaveReplicationData.Streams[0].BaseDescription.FrequencySettings);

					if (!EnumHasAnyFlags(Flags, ERestoreTestFlags::TestForAuthority))
					{
						return;
					}
					
					if (ClientInfo.Authority.Num() != 1)
					{
						AddError(TEXT("Expected client authority"));
						return;
					}
					TestEqual(TEXT("Owns correct stream"), ClientInfo.Authority[0].StreamId, RestoreStreamId);
					TestEqual(TEXT("Owns 2 objects"), ClientInfo.Authority[0].AuthoredObjects.Num(), 2);
					TestTrue(TEXT("Owns correct RestoredObject1"), ClientInfo.HasAuthority({ RestoreStreamId, RestoredObject1->TestObject }));
					TestTrue(TEXT("Owns correct RestoredObject2"), ClientInfo.HasAuthority({ RestoreStreamId, RestoredObject2->TestObject }));
				});
		}

		static FConcertClientInfo MakeClientInfo() { return FConcertClientInfo { .DeviceName = TEXT("MainDeviceName"), .DisplayName = TEXT("MainClientName") }; }
	END_DEFINE_SPEC(FRestoreContentWithNoPreExistingSpec);

	/** This tests the base functionality of restoring stream and authority content: when the client has an empty stream at the time of restore. */
	void FRestoreContentWithNoPreExistingSpec::Define()
	{
		BeforeEach([this]
		{
			WorkspaceMock = MakeShared<FReplicationWorkspaceCallInterceptorMock>();
			RestoredObject1 = MakeUnique<FObjectTestReplicator>(TEXT("Object1"));
			RestoredObject2 = MakeUnique<FObjectTestReplicator>(TEXT("Object2"));
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession, WorkspaceMock.ToSharedRef());
			Client = &Server->ConnectClient(MakeClientInfo());

			FConcertReplicationStream Stream = RestoredObject1->CreateStream(RestoreStreamId, EConcertObjectReplicationMode::SpecifiedRate, 21);
			RestoredObject2->AddToStream(Stream, EConcertObjectReplicationMode::SpecifiedRate, 42);
			LeaveReplicationData.Streams.Add(Stream);
			LeaveReplicationData.OwnedObjects.Add({ RestoreStreamId, RestoredObject1->TestObject });
			LeaveReplicationData.OwnedObjects.Add({ RestoreStreamId, RestoredObject2->TestObject });
			
			Client->JoinReplication();
		});
		AfterEach([this]
		{
			WorkspaceMock.Reset();
			Server.Reset();
			RestoredObject1.Reset();
			RestoredObject2.Reset();
			Client = nullptr;
			LeaveReplicationData = {};
		});

		Describe("Restores content correctly", [this]
		{
			// Try all permutations of flags
			const auto BuildTests = [this](EConcertReplicationAuthorityRestoreMode Mode)
			{
				It("StreamsAndAuthority", [this, Mode]
				{
					const EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority;
					RestoreStreamAndAuthorityThenTest({ .Flags = Flags, .AuthorityRestorationMode = Mode });
				});
				It("StreamsAndAuthority | RestoreOnTop", [this, Mode]
				{
					const EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority | EConcertReplicationRestoreContentFlags::RestoreOnTop;
					RestoreStreamAndAuthorityThenTest({ .Flags = Flags, .AuthorityRestorationMode = Mode });
				});
				It("StreamsAndAuthority | ValidateUniqueClient", [this, Mode]
				{
					const EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority | EConcertReplicationRestoreContentFlags::ValidateUniqueClient;
					RestoreStreamAndAuthorityThenTest({ .Flags = Flags, .AuthorityRestorationMode = Mode });
				});
				It("StreamsAndAuthority | RestoreOnTop | ValidateUniqueClient", [this, Mode]
				{
					const EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority | EConcertReplicationRestoreContentFlags::ValidateUniqueClient;
					RestoreStreamAndAuthorityThenTest({ .Flags = Flags, .AuthorityRestorationMode = Mode });
				});

				Describe("Streams only", [this, Mode]
				{
					AfterEach([this]
					{
						TestTrue(TEXT("Client thinks it has no authority"), Client->GetClientReplicationManager().GetClientOwnedObjects().IsEmpty());
						TestTrue(TEXT("Client thinks it has no sync control"), Client->GetClientReplicationManager().GetSyncControlledObjects().IsEmpty());
						
						Client->GetClientReplicationManager()
							.QueryClientInfo({ .ClientEndpointIds = { Client->GetEndpointId() } })
							.Next([this](FConcertReplication_QueryReplicationInfo_Response&& Response)
							{
								if (const FConcertQueriedClientInfo* ClientInfo = Response.ClientInfo.Find(Client->GetEndpointId()))
								{
									TestTrue(TEXT("No authority on server"), ClientInfo->Authority.IsEmpty());
								}
								else
								{
									AddError(TEXT("Streams were not restored"));
								}
							});
					});
					
					It("None", [this, Mode]
					{
						const EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::None;
						RestoreStreamAndAuthorityThenTest({ .Flags = Flags, .AuthorityRestorationMode = Mode }, ERestoreTestFlags::None);
					});
					It("RestoreOnTop", [this, Mode]
					{
						const EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::RestoreOnTop;
						RestoreStreamAndAuthorityThenTest({ .Flags = Flags, .AuthorityRestorationMode = Mode }, ERestoreTestFlags::None);
					});
					It("ValidateUniqueClient", [this, Mode]
					{
						const EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::ValidateUniqueClient;
						RestoreStreamAndAuthorityThenTest({ .Flags = Flags, .AuthorityRestorationMode = Mode }, ERestoreTestFlags::None);
					});
					It("RestoreOnTop | ValidateUniqueClient", [this, Mode]
					{
						const EConcertReplicationRestoreContentFlags Flags = EConcertReplicationRestoreContentFlags::RestoreOnTop | EConcertReplicationRestoreContentFlags::ValidateUniqueClient;
						RestoreStreamAndAuthorityThenTest({ .Flags = Flags, .AuthorityRestorationMode = Mode }, ERestoreTestFlags::None);
					});
				});
			};
			
			Describe("ExcludeAlreadyOwnedObjectPropertiesFromStream", [this, BuildTests]
			{
				BuildTests(EConcertReplicationAuthorityRestoreMode::ExcludeAlreadyOwnedObjectPropertiesFromStream);
			});
			Describe("IncludeAlreadyOwnedObjectPropertiesInStream", [this, BuildTests]
			{
				BuildTests(EConcertReplicationAuthorityRestoreMode::IncludeAlreadyOwnedObjectPropertiesInStream);
			});
			Describe("AllOrNothing", [this, BuildTests]
			{
				BuildTests(EConcertReplicationAuthorityRestoreMode::AllOrNothing);
			});
		});

		Describe("ClientInfo", [this]
		{
			It("Is contained when EConcertReplicationRestoreContentFlagsSendNewState is set", [this]
			{
				InsertActivityData();
				RestoreThenTestErrorCode(
					{ .Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority | EConcertReplicationRestoreContentFlags::SendNewState },
					EConcertReplicationRestoreErrorCode::Success
					)
				.Next([this](FConcertReplication_RestoreContent_Response&& Response)
				{
					const FConcertQueriedClientInfo& ClientInfo = Response.ClientInfo;
					if (ClientInfo.Streams.Num() != 1)
					{
						AddError(TEXT("Wrong streams"));
						return;
					}
					TestEqual(TEXT("Stream Content"), ClientInfo.Streams[0], LeaveReplicationData.Streams[0].BaseDescription);

					if (ClientInfo.Authority.Num() != 1)
					{
						AddError(TEXT("Wrong authority"));
						return;
					}
					TestEqual(TEXT("Authority StreamId"), ClientInfo.Authority[0].StreamId, RestoreStreamId);
					TestTrue(TEXT("Authority RestoredObject1"), ClientInfo.Authority[0].AuthoredObjects.Contains(RestoredObject1->TestObject));
					TestTrue(TEXT("Authority RestoredObject2"), ClientInfo.Authority[0].AuthoredObjects.Contains(RestoredObject2->TestObject));
				});
			});

			It("Is not contained when EConcertReplicationRestoreContentFlagsSendNewState is not set", [this]
			{
				InsertActivityData();
				RestoreThenTestErrorCode(
					{ .Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority },
					EConcertReplicationRestoreErrorCode::Success
					)
				.Next([this](FConcertReplication_RestoreContent_Response&& Response)
				{
					TestTrue(TEXT("No ClientInfo"), Response.ClientInfo.IsEmpty());
				});
			});
		});

		Describe("When another client has authority", [this]
		{
			BeforeEach([this]
			{
				InsertActivityData();
				
				FReplicationClient& OtherClient = Server->ConnectClient();
				OtherClient.JoinReplication(RestoredObject1->CreateSenderArgs());
				OtherClient.GetClientReplicationManager().TakeAuthorityOver({ RestoredObject1->TestObject });
			});
			
			It("When request has EConcertReplicationAuthorityRestoreMode::ExcludeAlreadyOwnedObjectsFromStream, the object is excluded from the stream", [this]
			{
				RestoreThenTestErrorCode(
					{ .Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority, .AuthorityRestorationMode = EConcertReplicationAuthorityRestoreMode::ExcludeAlreadyOwnedObjectPropertiesFromStream },
					EConcertReplicationRestoreErrorCode::Success
				);

				Client->GetClientReplicationManager()
					.QueryClientInfo({ { Client->GetEndpointId() } })
					.Next([this](FConcertReplication_QueryReplicationInfo_Response&& Response)
					{
						const FConcertQueriedClientInfo& ClientInfo = Response.ClientInfo[Client->GetEndpointId()];
						if (ClientInfo.Authority.Num() != 1)
						{
							AddError(TEXT("Expected client authority"));
							return;
						}
						TestEqual(TEXT("Owns correct stream"), ClientInfo.Authority[0].StreamId, RestoreStreamId);
						TestEqual(TEXT("Owns 1 objects"), ClientInfo.Authority[0].AuthoredObjects.Num(), 1);
						TestTrue(TEXT("Owns correct RestoredObject2"), ClientInfo.HasAuthority({ RestoreStreamId, RestoredObject2->TestObject }));
					
						TestEqual(TEXT("No streams"), ClientInfo.Streams.Num(), 1);
						FConcertBaseStreamInfo ExpectedStream = LeaveReplicationData.Streams[0].BaseDescription;
						ExpectedStream.ReplicationMap.ReplicatedObjects.Remove(RestoredObject1->TestObject);
						ExpectedStream.FrequencySettings.ObjectOverrides.Remove(RestoredObject1->TestObject);
						const FConcertBaseStreamInfo* RestoreStreamInfo = ClientInfo.Streams.FindByPredicate([this](const FConcertBaseStreamInfo& StreamInfo){ return StreamInfo.Identifier == RestoreStreamId; });
						TestTrue(TEXT("Stream content"), RestoreStreamInfo && *RestoreStreamInfo == ExpectedStream);
					});
			});
			It("When request has EConcertReplicationAuthorityRestoreMode::IncludeAlreadyOwnedObjectsInStream, the object is included in the stream", [this]
			{
				RestoreThenTestErrorCode(
					{ .Flags = EConcertReplicationRestoreContentFlags::StreamsAndAuthority, .AuthorityRestorationMode = EConcertReplicationAuthorityRestoreMode::IncludeAlreadyOwnedObjectPropertiesInStream },
					EConcertReplicationRestoreErrorCode::Success
				);
				
				Client->GetClientReplicationManager()
					.QueryClientInfo({ { Client->GetEndpointId() } })
					.Next([this](FConcertReplication_QueryReplicationInfo_Response&& Response)
					{
						const FConcertQueriedClientInfo& ClientInfo = Response.ClientInfo[Client->GetEndpointId()];
						if (ClientInfo.Authority.Num() != 1)
						{
							AddError(TEXT("Expected client authority"));
							return;
						}
						TestEqual(TEXT("Owns correct stream"), ClientInfo.Authority[0].StreamId, RestoreStreamId);
						TestEqual(TEXT("Owns 1 object"), ClientInfo.Authority[0].AuthoredObjects.Num(), 1);
						TestTrue(TEXT("Owns correct RestoredObject2"), ClientInfo.HasAuthority({ RestoreStreamId, RestoredObject2->TestObject }));
						
						TestEqual(TEXT("1 stream"), ClientInfo.Streams.Num(), 1);
						const FConcertBaseStreamInfo* RestoreStreamInfo = ClientInfo.Streams.FindByPredicate([this](const FConcertBaseStreamInfo& StreamInfo){ return StreamInfo.Identifier == RestoreStreamId; });
						TestTrue(TEXT("Stream content"), RestoreStreamInfo && *RestoreStreamInfo == LeaveReplicationData.Streams[0].BaseDescription);
					});
			});
			It("When request has EConcertReplicationAuthorityRestoreMode::AllOrNothing, the request fails", [this]
			{
				RestoreThenTestErrorCode({ .AuthorityRestorationMode = EConcertReplicationAuthorityRestoreMode::AllOrNothing }, EConcertReplicationRestoreErrorCode::AuthorityConflict);
				RestoreThenTestErrorCode(
					{ .Flags = EConcertReplicationRestoreContentFlags::None, .AuthorityRestorationMode = EConcertReplicationAuthorityRestoreMode::AllOrNothing },
					EConcertReplicationRestoreErrorCode::AuthorityConflict
				);
			});
		});
		
		It("When there is the 'same' client and request has ValidateUniqueClient flag, the error code is NameConflict", [this]
		{
			InsertActivityData();
			Server->ConnectClient(MakeClientInfo());
			RestoreThenTestErrorCode(
				{ .Flags = EConcertReplicationRestoreContentFlags::ValidateUniqueClient },
				EConcertReplicationRestoreErrorCode::NameConflict
				);
		});
		It("When ActivityId references unknown activity, the error code is InvalidActivity.", [this]
		{
			constexpr int64 InvalidActivityId = 42;
			RestoreThenTestErrorCode(
				{ .ActivityId = { InvalidActivityId } },
				EConcertReplicationRestoreErrorCode::NoSuchActivity
				);
		});
		It("When ActivityId is left unset and there is no activity, the error code is Success.", [this]
		{
			RestoreThenTestErrorCode(
				{ },
				EConcertReplicationRestoreErrorCode::Success
				);
		});
	}
	
	BEGIN_DEFINE_SPEC(FRestoreContentPropertiesSpec, "VirtualProduction.Concert.Replication.RestoreContent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		/** Detects calls into the workspace */
		TSharedPtr<FReplicationWorkspaceCallInterceptorMock> WorkspaceMock;
		FConcertSyncReplicationPayload_LeaveReplication LeaveReplicationData;

		/** The stream to restore */
		TUniquePtr<FObjectTestReplicator> RestoreObjectReplicator;
		/** The stream that the client joins with */
		TUniquePtr<FObjectTestReplicator> JoinObjectReplicator;
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
	
		FGuid RestoreStreamId = FGuid::NewGuid();
		FGuid JoinStreamId = FGuid::NewGuid();

		static FConcertClientInfo MakeClientInfo() { return FConcertClientInfo { .DeviceName = TEXT("MainDeviceName"), .DisplayName = TEXT("MainClientName") }; }
		static FName OriginalObjectName() { return TEXT("OriginalObjectName"); }
		static FName RestoreObjectName() { return TEXT("OriginalObjectName"); }
	END_DEFINE_SPEC(FRestoreContentPropertiesSpec);

	/** This tests that a client's stream and authority can be restored when EConcertSyncSessionFlags::ShouldEnableReplicationActivities is set. */
	void FRestoreContentPropertiesSpec::Define()
	{
		// TODO: Replaces correctly
		// TODO: Test RestoreOnTop combines correctly (in same stream)
		// TODO: Test client 2 has authority over non-overlapping properties
		/*
		It("Restores correctly (EConcertReplicationRestoreContentFlags::RestoreOnTop)", [this]
		{
			TestErrorCode(
				{ .Flags = EConcertReplicationRestoreContentFlags::Default | EConcertReplicationRestoreContentFlags::RestoreOnTop },
				EConcertReplicationRestoreErrorCode::Success
			);
			TestHasRestored(ERestoreTestFlags::ShouldHaveAuthority);
		});*/
	}
}
