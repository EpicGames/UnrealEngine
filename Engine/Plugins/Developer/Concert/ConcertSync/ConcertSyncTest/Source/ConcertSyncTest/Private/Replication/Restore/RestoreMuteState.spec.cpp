// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Mocks/ReplicationWorkspaceCallInterceptorMock.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/ReplicationTestInterface.h"
#include "Util/ScopedSessionDatabase.h"

namespace UE::ConcertSyncTests::Replication::RestoreContent
{
	BEGIN_DEFINE_SPEC(FRestoreMuteStateSpec, "VirtualProduction.Concert.Replication.RestoreContent.Mute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FScopedSessionDatabase> SessionDatabase;
		TSharedPtr<ConcertSyncServer::Replication::IReplicationWorkspace> ReplicationWorkspace;
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;
	
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
	
		FReplicationClient& ConnectClient()
		{
			FReplicationClient& CreatedClient = Server->ConnectClient();
			SessionDatabase->SetEndpoint(CreatedClient.GetEndpointId(), { CreatedClient.GetClientInfo() });
			return CreatedClient;
		}
	END_DEFINE_SPEC(FRestoreMuteStateSpec);

	/** This tests that mute state is correctly restored. */
	void FRestoreMuteStateSpec::Define()
	{
		BeforeEach([this]
		{
			SessionDatabase = MakeUnique<FScopedSessionDatabase>(*this);
			const auto FindSessionClient = [this](const FGuid& EndpointId) -> TOptional<FConcertSessionClientInfo>
			{
				FConcertSessionClientInfo Info;
				const bool bFound = Server->GetServerSessionMock()->FindSessionClient(EndpointId, Info);
				return bFound ? Info : TOptional<FConcertSessionClientInfo>{};
			};
			const auto ShouldIgnoreClientActivityOnRestore = [](const FGuid& EndpointId){ return false; };
			ReplicationWorkspace = ConcertSyncServer::TestInterface::CreateReplicationWorkspace(*SessionDatabase, FindSessionClient, ShouldIgnoreClientActivityOnRestore);
			
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession, ReplicationWorkspace.ToSharedRef());
			Client = &ConnectClient();

			Client->JoinReplication(ObjectReplicator->CreateSenderArgs());
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
			SessionDatabase.Reset();
			ReplicationWorkspace.Reset();
		});

		Describe("If an object had been muted before rejoining session", [this]
		{
			BeforeEach([this]
			{
				IConcertClientReplicationManager& ReplicationManager = Client->GetClientReplicationManager();
				
				ReplicationManager.MuteObjects({ ObjectReplicator->TestObject });
				ReplicationManager.LeaveReplicationSession();
				ReplicationManager.JoinReplicationSession({});
			});
			
			It("EConcertReplicationRestoreContentFlags::StreamsAndAuthority skips mute restoration", [this]
			{
				IConcertClientReplicationManager& ReplicationManager = Client->GetClientReplicationManager();
				ReplicationManager.RestoreContent({ EConcertReplicationRestoreContentFlags::StreamsAndAuthority });
				
				bool bReceivedResponse = false;
				ReplicationManager.QueryMuteState()
					.Next([this, &bReceivedResponse](FConcertReplication_QueryMuteState_Response&& Response)
					{
						bReceivedResponse = true;
						TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 0);
					});
				TestTrue(TEXT("Received response"), bReceivedResponse);
			});

			It("EConcertReplicationRestoreContentFlags::All restores mute state", [this]
			{
				IConcertClientReplicationManager& ReplicationManager = Client->GetClientReplicationManager();
				ReplicationManager.RestoreContent({ EConcertReplicationRestoreContentFlags::All });
				
				bool bReceivedResponse = false;
				ReplicationManager.QueryMuteState()
					.Next([this, &bReceivedResponse](FConcertReplication_QueryMuteState_Response&& Response)
					{
						bReceivedResponse = true;
						TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 1);
						TestTrue(TEXT("ExplicitlyMutedObjects.Contains(TestObject)"), Response.ExplicitlyMutedObjects.Contains(ObjectReplicator->TestObject));
					});
				TestTrue(TEXT("Received response"), bReceivedResponse);
			});
		});

		It("If object had been muted and then unmuted, then the object is not muted when rejoining", [this]
		{
			IConcertClientReplicationManager& ReplicationManager = Client->GetClientReplicationManager();
			ReplicationManager.MuteObjects({ ObjectReplicator->TestObject });
			ReplicationManager.UnmuteObjects({ ObjectReplicator->TestObject });
			ReplicationManager.LeaveReplicationSession();
			ReplicationManager.JoinReplicationSession({});
			ReplicationManager.RestoreContent({ EConcertReplicationRestoreContentFlags::All });
				
			bool bReceivedResponse = false;
			ReplicationManager.QueryMuteState()
				.Next([this, &bReceivedResponse](FConcertReplication_QueryMuteState_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("ExplicitlyMutedObjects.Num()"), Response.ExplicitlyMutedObjects.Num(), 0);
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
		});
	}
}
