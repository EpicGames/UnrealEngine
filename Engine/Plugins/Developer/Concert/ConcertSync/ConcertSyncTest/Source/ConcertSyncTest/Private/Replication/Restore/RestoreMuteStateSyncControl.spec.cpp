// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Mocks/ReplicationWorkspaceCallInterceptorMock.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/ReplicationTestInterface.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"
#include "Util/ScopedSessionDatabase.h"

namespace UE::ConcertSyncTests::Replication::RestoreContent
{
	BEGIN_DEFINE_SPEC(FRestoreMuteStateSyncControlSpec, "VirtualProduction.Concert.Replication.RestoreContent.Mute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FScopedSessionDatabase> SessionDatabase;
		TSharedPtr<ConcertSyncServer::Replication::IReplicationWorkspace> ReplicationWorkspace;
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;
	
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* SenderClient = nullptr;
		FReplicationClient* ReceiverClient = nullptr;

		FGuid StreamId = FGuid::NewGuid();
	
		FReplicationClient& ConnectClient()
		{
			FReplicationClient& CreatedClient = Server->ConnectClient();
			SessionDatabase->SetEndpoint(CreatedClient.GetEndpointId(), { CreatedClient.GetClientInfo() });
			return CreatedClient;
		}
	END_DEFINE_SPEC(FRestoreMuteStateSyncControlSpec);

	/** This tests that sync control is correct when restoring mute state. */
	void FRestoreMuteStateSyncControlSpec::Define()
	{
		BeforeEach([this]
		{
			SessionDatabase = MakeUnique<FScopedSessionDatabase>(*this);
			const auto FindSessionClient = [this](const FGuid& EndpointId) -> TOptional<FConcertSessionClientInfo>
			{
				FConcertSessionClientInfo Info;
				const bool bFound = Server->GetServerSessionMock()->FindSessionClient(EndpointId, Info);
				return bFound ? Info : TOptional<FConcertSessionClientInfo>{};
			};
			const auto ShouldIgnoreClientActivityOnRestore = [](const FGuid& EndpointId){ return false; };
			ReplicationWorkspace = ConcertSyncServer::TestInterface::CreateReplicationWorkspace(*SessionDatabase, FindSessionClient, ShouldIgnoreClientActivityOnRestore);
			
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this, EConcertSyncSessionFlags::Default_MultiUserSession, ReplicationWorkspace.ToSharedRef());
			SenderClient = &ConnectClient();
			ReceiverClient = &ConnectClient();

			SenderClient->JoinReplication(ObjectReplicator->CreateSenderArgs(StreamId));
			ReceiverClient->JoinReplicationAsListener({ ObjectReplicator->TestObject });
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
			SessionDatabase.Reset();
			ReplicationWorkspace.Reset();
		});

		It("If object had been muted, rejoining session does not grant sync control", [this]
		{
			IConcertClientReplicationManager& ReplicationManager = SenderClient->GetClientReplicationManager();
			ReplicationManager.TakeAuthorityOver({ ObjectReplicator->TestObject });
			ReplicationManager.MuteObjects({ ObjectReplicator->TestObject });
			ReplicationManager.LeaveReplicationSession();
			ReplicationManager.JoinReplicationSession({});

			const FConcertObjectInStreamID ObjectInStreamID{ StreamId, ObjectReplicator->TestObject };
			bool bReceivedResponse = false;
			ReplicationManager.RestoreContent({ EConcertReplicationRestoreContentFlags::All })
				.Next([this, &ObjectInStreamID, &bReceivedResponse](FConcertReplication_RestoreContent_Response&& Response)
				{
					bReceivedResponse = true;
					TestEqual(TEXT("NewControlStates.Num()"), Response.SyncControl.NewControlStates.Num(), 1);
					
					const bool* bHasSyncControl = Response.SyncControl.NewControlStates.Find(ObjectInStreamID);
					TestTrue(TEXT("Object does not have sync control"), bHasSyncControl && !*bHasSyncControl);
				});
			TestTrue(TEXT("Received response"), bReceivedResponse);
			TestFalse(TEXT("!HasSyncControl(TestObject)"), ReplicationManager.HasSyncControl(ObjectInStreamID));
		});
		
		Describe("If object had not been muted", [this]
		{
			BeforeEach([this]
			{
				IConcertClientReplicationManager& ReplicationManager = SenderClient->GetClientReplicationManager();
				ReplicationManager.TakeAuthorityOver({ ObjectReplicator->TestObject });
				ReplicationManager.MuteObjects({ ObjectReplicator->TestObject });
				ReplicationManager.UnmuteObjects({ ObjectReplicator->TestObject });
				ReplicationManager.LeaveReplicationSession();
				ReplicationManager.JoinReplicationSession({});
			});
			
			It("Rejoining session grant sync control", [this]
			{
				IConcertClientReplicationManager& ReplicationManager = SenderClient->GetClientReplicationManager();
				const FConcertObjectInStreamID ObjectInStreamID{ StreamId, ObjectReplicator->TestObject };
				
				bool bReceivedResponse = false;
				ReplicationManager
					.RestoreContent({ EConcertReplicationRestoreContentFlags::All })
					.Next([this, &ObjectInStreamID, &bReceivedResponse](FConcertReplication_RestoreContent_Response&& Response)
					{
						bReceivedResponse = true;
						TestEqual(TEXT("SyncControl.IsEmpty()"), Response.SyncControl.NewControlStates.Num(), 1);

						const bool* bHasSyncControl = Response.SyncControl.NewControlStates.Find(ObjectInStreamID);
						TestTrue(TEXT("Object has sync control"), bHasSyncControl && *bHasSyncControl);
					});
				TestTrue(TEXT("Received response"), bReceivedResponse);
				TestTrue(TEXT("HasSyncControl(TestObject)"), ReplicationManager.HasSyncControl(ObjectInStreamID));
			});
			
			It("Can replicate object after rejoining", [this]
			{
				SenderClient->GetClientReplicationManager().RestoreContent({ EConcertReplicationRestoreContentFlags::All });

				SenderClient->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
				ReceiverClient->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
				ObjectReplicator->SimulateSendObjectToReceiver(*this, { *SenderClient, *Server, *ReceiverClient }, { StreamId });
				ObjectReplicator->TestValuesWereReplicated(*this);
			});
		});
	}
}
