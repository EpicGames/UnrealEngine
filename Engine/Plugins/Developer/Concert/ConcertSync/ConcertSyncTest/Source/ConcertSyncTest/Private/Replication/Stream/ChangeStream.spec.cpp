// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Mocks/ReplicationWorkspaceCallInterceptorMock.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/ReplicationTestInterface.h"
#include "Util/ScopedSessionDatabase.h"

namespace UE::ConcertSyncTests::Replication::RestoreContent
{
	BEGIN_DEFINE_SPEC(FChangeStreamSpec, "VirtualProduction.Concert.Replication.Stream", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> FooObject;
		TUniquePtr<FObjectTestReplicator> BarObject;
	
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;
	
		const FGuid StreamId = FGuid::NewGuid();
	END_DEFINE_SPEC(FChangeStreamSpec);

	/**
	 * This tests use cases that only issue FConcertReplication_ChangeStream_Request.
	 * 
	 * Old tests are in StreamRequestTests_x.cpp files of the same folder.
	 * In the future, we want to move those over to the spec test format, too.
	 */
	void FChangeStreamSpec::Define()
	{
		BeforeEach([this]
		{
			FooObject = MakeUnique<FObjectTestReplicator>(TEXT("Foo"));
			BarObject = MakeUnique<FObjectTestReplicator>(TEXT("Bar"));
			
			Server = MakeUnique<FReplicationServer>(*this);
			Client = &Server->ConnectClient();
			Client->JoinReplication();
		});
		AfterEach([this]
		{
			Server.Reset();
			BarObject.Reset();
			FooObject.Reset();
		});

		It("Replace stream content", [this]
		{
			IConcertClientReplicationManager& ReplicationManager = Client->GetClientReplicationManager();
			ReplicationManager.ChangeStream({ .StreamsToAdd = { FooObject->CreateStream(StreamId) } });

			const FConcertReplicationStream BarStream = BarObject->CreateStream(StreamId);
			const FConcertReplication_ChangeStream_Request ReplaceWithBarRequest
			{
				.StreamsToAdd = { BarStream },
				.StreamsToRemove = { StreamId  }
			};
			bool bReceivedStreamResponse = false;
			ReplicationManager
				.ChangeStream(ReplaceWithBarRequest)
				.Next([this, &bReceivedStreamResponse](FConcertReplication_ChangeStream_Response&& Response)
				{
					bReceivedStreamResponse = true;
					TestTrue(TEXT("Success"), Response.IsSuccess());
				});
			TestTrue(TEXT("bReceivedResponse"), bReceivedStreamResponse);

			bool bReceivedQueryResponse = false;
			ReplicationManager
				.QueryClientInfo({ .ClientEndpointIds = { Client->GetEndpointId() } })
				.Next([this, &bReceivedQueryResponse, &BarStream](FConcertReplication_QueryReplicationInfo_Response&& Response)
				{
					bReceivedQueryResponse = true;
					
					const FConcertQueriedClientInfo* ClientInfo = Response.ClientInfo.Find(Client->GetEndpointId());
					TestTrue(TEXT("Client info"), ClientInfo && ClientInfo->Streams.Num() == 1 && ClientInfo->Streams[0] == BarStream.BaseDescription);
					
				});
			TestTrue(TEXT("bReceivedQueryResponse"), bReceivedStreamResponse);
		});

		It("When a request tries to create an empty stream, the request fails", [this]
		{
			AddExpectedError(TEXT("Rejecting ChangeStream request from"));
			
			bool bReceivedResponse = false;
			IConcertClientReplicationManager& ReplicationManager = Client->GetClientReplicationManager();
			ReplicationManager
				.ChangeStream({ .StreamsToAdd = { FConcertReplicationStream{ .BaseDescription = { .Identifier = StreamId } } } })
				.Next([this, &bReceivedResponse](FConcertReplication_ChangeStream_Response&& Response)
				{
					bReceivedResponse = true;
					TestFalse(TEXT("Failure"), Response.IsSuccess());
					TestEqual(TEXT("FailedStreamCreation.Num()"), Response.FailedStreamCreation.Num(), 1);
					TestTrue(TEXT("FailedStreamCreation.Contains(StreamId)"), Response.FailedStreamCreation.Contains(StreamId));
				});
			TestTrue(TEXT("bReceivedResponse"), bReceivedResponse);
		});
	}
}
