// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/Mocks/ReplicationWorkspaceCallInterceptorMock.h"
#include "Replication/Util/Spec/ReplicationClient.h"
#include "Replication/Util/Spec/ReplicationServer.h"
#include "Replication/Util/Spec/ObjectTestReplicator.h"

#include "Misc/AutomationTest.h"
#include "Replication/ReplicationTestInterface.h"
#include "Util/ScopedSessionDatabase.h"

namespace UE::ConcertSyncTests::Replication::RestoreContent
{
	BEGIN_DEFINE_SPEC(FStreamAuthorityInterplaySpec, "VirtualProduction.Concert.Replication.Stream", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		TUniquePtr<FObjectTestReplicator> ObjectReplicator;
	
		TUniquePtr<FReplicationServer> Server;
		FReplicationClient* Client = nullptr;

		const FGuid StreamId = FGuid::NewGuid();
	END_DEFINE_SPEC(FStreamAuthorityInterplaySpec);

	/**
	 * This tests that stream structure and authority correctly interact when issuing a FConcertReplication_ChangeStream_Request.
	 * 
	 * Old tests are in StreamRequestTests_x.cpp files of the same folder.
	 * In the future, we want to move those over to the spec test format, too.
	 */
	void FStreamAuthorityInterplaySpec::Define()
	{
		BeforeEach([this]
		{
			ObjectReplicator = MakeUnique<FObjectTestReplicator>();
			Server = MakeUnique<FReplicationServer>(*this);
			Client = &Server->ConnectClient();
			Client->JoinReplication();
			
		});
		AfterEach([this]
		{
			Server.Reset();
			ObjectReplicator.Reset();
		});

		It("Removing stream also removes authority locally", [this]
		{
			IConcertClientReplicationManager& ReplicationManager = Client->GetClientReplicationManager();
			ReplicationManager.ChangeStream({ .StreamsToAdd = { ObjectReplicator->CreateStream(StreamId) } });
			ReplicationManager.TakeAuthorityOver({ ObjectReplicator->TestObject });
			ReplicationManager.ChangeStream({ .StreamsToRemove = { StreamId } });

			TestEqual(TEXT("Clients updated local authority cache to 0"), ReplicationManager.GetClientOwnedObjects().Num(), 0);
		});
	}
}
