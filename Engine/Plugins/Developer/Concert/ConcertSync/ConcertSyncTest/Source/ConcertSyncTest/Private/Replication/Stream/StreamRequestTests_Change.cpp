// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/ClientServerCommunicationTest.h"

#include "Replication/Data/ReplicationStream.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Util/ChangeStreamsTestBase.h"
#include "Replication/Util/SendReceiveObjectTestBase.h"
#include "Replication/TestReflectionObject.h"

#include "Misc/AutomationTest.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::ConcertSyncTests::Replication::Stream
{
	/** Simple case of FConcertChangeStream_Request where the requesting client has authority and thus cannot generate any authority conflicts. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FSimpleChangeStream, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.SimpleChangeStream", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FSimpleChangeStream::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncClient::Replication;

		// 1. Start up client with float property stream
		UTestReflectionObject* TestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		auto[InitialStreamId, InitialStream] = CreateFloatPropertyStream(*TestObject);
		SenderArgs.Streams = { InitialStream };
		SetUpClientAndServer();

		// 2.1 Modify existing stream
		FConcertPropertySelection& ChangedInitialProperties = GetPropertySelection(InitialStream, *TestObject);
		AddVectorProperty(ChangedInitialProperties);
		FConcertReplication_ChangeStream_Request ModifyRequest;
		ModifyRequest.ObjectsToPut.Add({ InitialStreamId, TestObject }, { ChangedInitialProperties });
		ChangeStreamForSenderClientAndValidate(TEXT("ModifyRequest"), ModifyRequest, { InitialStream.BaseDescription });

		// 2.2 Add new stream
		auto[DynamicStreamId, DynamicStream] = CreateFloatPropertyStream(*TestObject);
		FConcertReplication_ChangeStream_Request DynamicCreationRequest;
		DynamicCreationRequest.StreamsToAdd.Add(DynamicStream);
		ChangeStreamForSenderClientAndValidate(TEXT("DynamicCreationRequest"), DynamicCreationRequest, { InitialStream.BaseDescription, DynamicStream.BaseDescription });

		// 2.3 Remove stream
		FConcertReplication_ChangeStream_Request RemoveStreamRequest;
		RemoveStreamRequest.StreamsToRemove.Add(InitialStreamId);
		ChangeStreamForSenderClientAndValidate(TEXT("RemoveStreamRequest"), RemoveStreamRequest, { DynamicStream.BaseDescription });

		// 2.4 Remove object > removes stream implicitly
		FConcertReplication_ChangeStream_Request RemoveObjectRequest;
		RemoveObjectRequest.ObjectsToRemove.Add({ DynamicStreamId, TestObject });
		ChangeStreamForSenderClientAndValidate(TEXT("RemoveObjectRequest"), RemoveObjectRequest, {});
		
		return true;
	}

	/** Case of FConcertChangeStream_Request in which requester has authority over an object and creates two streams which have authority over overlapping properties. This is allowed. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FChangeStreamWithOverlappingProperties, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.ChangeStreamWithOverlappingProperties", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FChangeStreamWithOverlappingProperties::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncClient::Replication;

		// 1. Start client without any streams
		UTestReflectionObject* TestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		SetUpClientAndServer();
		
		// 2.1 Add stream with float property
		auto[FloatStreamID, FloatStream] = CreateFloatPropertyStream(*TestObject);
		FConcertReplication_ChangeStream_Request CreateFloatStreamRequest;
		CreateFloatStreamRequest.StreamsToAdd.Add(FloatStream);
		ChangeStreamForSenderClientAndValidate(TEXT("CreateFloatStreamRequest"), CreateFloatStreamRequest, { FloatStream.BaseDescription });
		// 2.2 Add a stream with vector property
		auto[VectorFloatStreamID, VectorFloatStream] = CreateVectorPropertyStream(*TestObject);
		FConcertReplication_ChangeStream_Request CreateVectorFloatStreamRequest; 
		CreateVectorFloatStreamRequest.StreamsToAdd.Add(VectorFloatStream);
		ChangeStreamForSenderClientAndValidate(TEXT("CreateVectorFloatStreamRequest"), CreateVectorFloatStreamRequest, { FloatStream.BaseDescription, VectorFloatStream.BaseDescription });

		// 2.3 Take authority over both streams
		bool bTookAuthority = false;
		FConcertReplication_ChangeAuthority_Request TakeAuthorityRequest;
		TakeAuthorityRequest.TakeAuthority.Add(TestObject, FConcertStreamArray{{ FloatStreamID, VectorFloatStreamID }});
		ClientReplicationManager_Sender->RequestAuthorityChange({ TakeAuthorityRequest })
			.Next([this, &bTookAuthority](FConcertReplication_ChangeAuthority_Response&& Response)
			{
				bTookAuthority = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestTrue(TEXT("Authority request > Success"), Response.RejectedObjects.IsEmpty());
			});
		TestTrue(TEXT("Authority request > Received response"), bTookAuthority);

		// 2.4 Appending the overlapping float property works
		FConcertReplication_ChangeStream_Request AppendFloatRequest;
		FConcertPropertySelection& NewSelection = GetPropertySelection(VectorFloatStream, *TestObject);
		AddFloatProperty(NewSelection);
		AppendFloatRequest.ObjectsToPut.Add(FConcertObjectInStreamID{ VectorFloatStreamID, TestObject }, FConcertReplication_ChangeStream_PutObject{ NewSelection });
		ChangeStreamForSenderClientAndValidate(TEXT("AppendFloatRequest"), AppendFloatRequest, { FloatStream.BaseDescription, VectorFloatStream.BaseDescription });
		
		return true;
	}

	/**
	 * The requester and client have authority over separate properties on the same object.
	 * The request would cause the requester's existing authority to overlap with the other client, which is rejected.
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FRejectChangeStreamWithConflictingAuthority, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.RejectChangeStreamWithConflictingAuthority", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FRejectChangeStreamWithConflictingAuthority::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncClient::Replication;

		// 1. Start client without any streams
		UTestReflectionObject* TestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		SetUpClientAndServer();
		
		// 2.1 Add stream with float property
		auto[SenderStreamID, SenderStream] = CreateFloatPropertyStream(*TestObject);
		FConcertReplication_ChangeStream_Request CreateFloatStreamRequest;
		CreateFloatStreamRequest.StreamsToAdd.Add(SenderStream);
		ChangeStreamForSenderClientAndValidate(TEXT("CreateFloatStreamRequest"), CreateFloatStreamRequest, { SenderStream.BaseDescription });

		// 2.2 Have Sender take authority over the float property
		bool bSenderTookAuthority = false;
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObject })
			.Next([this, &bSenderTookAuthority](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bSenderTookAuthority = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestEqual(TEXT("Sender > No rejection taking authority"), Response.RejectedObjects.Num(), 0); 
			});
		TestTrue(TEXT("Sender > Received response taking authority"), bSenderTookAuthority);

		// 2.3 Have Receiver create vector stream
		auto[ReceiverStreamID, ReceiverStream] = CreateVectorPropertyStream(*TestObject);
		FConcertReplication_ChangeStream_Request CreateVectorStreamRequest; 
		CreateVectorStreamRequest.StreamsToAdd.Add(ReceiverStream);
		bool bAddedVectorStream = false;
		ClientReplicationManager_Receiver->ChangeStream(CreateVectorStreamRequest)
			.Next([this, &bAddedVectorStream](FConcertReplication_ChangeStream_Response&& Response)
			{
				bAddedVectorStream = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestTrue(TEXT("Vector Stream > Success"), Response.IsSuccess());
			});
		TestTrue(TEXT("Vector Stream> Received response"), bAddedVectorStream);
		// For simplicity, we don't do any further validation that other clients can see the change
		
		// 2.4 Have Receiver take authority over vector stream
		bool bReceiverTookAuthority = false;
		ClientReplicationManager_Receiver->TakeAuthorityOver({ TestObject })
			.Next([this, &bReceiverTookAuthority](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bReceiverTookAuthority = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestEqual(TEXT("Receiver > No rejection taking authority"), Response.RejectedObjects.Num(), 0); 
			});
		TestTrue(TEXT("Receiver > Received response taking authority"), bReceiverTookAuthority);

		// 2.5 Receiver is rejected from modifying the stream to include the Float property because the Sender has authority over it
		const FConcertObjectInStreamID TestObjectInReceiverStreamId { ReceiverStreamID, TestObject };
		FConcertReplication_ChangeStream_Request AddFloatToStreamRequest;
		FConcertPropertySelection NewSelection = GetPropertySelection(ReceiverStream, *TestObject);
		AddFloatProperty(NewSelection);
		AddFloatToStreamRequest.ObjectsToPut.Add(TestObjectInReceiverStreamId, { NewSelection });
		bool bReceivedResponseAddingFloat = false;
		
		// Server logs a warning when rejecting - avoid the test being marked with a warning.
		AddExpectedError(TEXT("Rejecting ChangeStream"));
		ClientReplicationManager_Receiver->ChangeStream(AddFloatToStreamRequest)
			.Next([this, TestObject, TheSenderStreamID = SenderStreamID, &TestObjectInReceiverStreamId, &bReceivedResponseAddingFloat](FConcertReplication_ChangeStream_Response&& Response)
			{
				bReceivedResponseAddingFloat = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestEqual(TEXT("Append Float > 1 conflict"), Response.AuthorityConflicts.Num(), 1);
				if (const FConcertReplicatedObjectId* ConflictingObject = Response.AuthorityConflicts.Find(TestObjectInReceiverStreamId))
				{
					TestEqual(TEXT("Append float > Conflict > Sender Stream correct"), ConflictingObject->StreamId, TheSenderStreamID);
					TestEqual(TEXT("Append float > Conflict > Object correct"), ConflictingObject->Object, FSoftObjectPath(TestObject));
					TestEqual(TEXT("Append float > Conflict > Endpoint ID correct"), ConflictingObject->SenderEndpointId, Client_Sender->ClientSessionMock->GetSessionClientEndpointId());
				}
				else
				{
					AddError(TEXT("Expected to find an authority object conflict for TestObject"));
				}
				TestTrue(TEXT("Append Float > Failure"), Response.IsFailure());
			});
		TestTrue(TEXT("Append Float > Received response"), bReceivedResponseAddingFloat);
		
		return true;  
	}

	/** Tests that no changes are made if a sub-step of FConcertChangeStream_Request causes an error. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FChangeStreamIsAtmoic, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.ChangeStreamIsAtmoic", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FChangeStreamIsAtmoic::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncClient::Replication;

		// 1. Start client without any streams
		UTestReflectionObject* TestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		SetUpClientAndServer();
		
		// 2.1 Add stream with float property
		auto[StreamId, Stream] = CreateFloatPropertyStream(*TestObject);
		FConcertReplication_ChangeStream_Request CreateStreamRequest;
		CreateStreamRequest.StreamsToAdd.Add(Stream);
		ChangeStreamForSenderClientAndValidate(TEXT("CreateStreamRequest"), CreateStreamRequest, { Stream.BaseDescription });

		// 2.2 Make a request that will fail and check that no changes were made to the original stream
		FConcertReplication_ChangeStream_Request InvalidRequest;
		FConcertPropertySelection NewSelection = GetPropertySelection(Stream, *TestObject);
		AddFloatProperty(NewSelection);
		InvalidRequest.ObjectsToPut.Add(FConcertObjectInStreamID{ StreamId, TestObject}, FConcertReplication_ChangeStream_PutObject{ NewSelection });
		InvalidRequest.StreamsToAdd.Add(Stream); // This will make it fail due to pre-existing stream ID

		// Server logs a warning when rejecting - avoid the test being marked with a warning.
		AddExpectedError(TEXT("Rejecting ChangeStream"));
		
		bool bReceivedChangeStreamResponse = false;
		ClientReplicationManager_Sender->ChangeStream(InvalidRequest)
			.Next([this, &bReceivedChangeStreamResponse](FConcertReplication_ChangeStream_Response&& Response)
			{
				bReceivedChangeStreamResponse = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestTrue(TEXT("Invalid Request > Modify Stream > Failure"), Response.IsFailure());
			});
		
		TestTrue(TEXT("Invalid Request > Modify Stream > Received response"), bReceivedChangeStreamResponse);
		ValidateSenderClientStreams(TEXT("Invalid Request"), { Stream.BaseDescription });
		
		return true;
	}

	/** Removing an object from a stream over which the requester has authority, clears the authority the client had. Tests that another client can now claim authority over the previously authored properties. */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FRemovingObjectClearsAuthority, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.RemovingObjectClearsAuthority", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FRemovingObjectClearsAuthority::RunTest(const FString& Parameters)
	{
		using namespace ConcertSyncClient::Replication;

		// 1. Start clients with equivalent streams
		UTestReflectionObject* TestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		auto[SenderStreamId, SenderStream] = CreateFloatPropertyStream(*TestObject);
		auto[ReceiverStreamId, ReceiverStream] = CreateFloatPropertyStream(*TestObject);
		SenderArgs.Streams = { SenderStream };
		ReceiverArgs.Streams = { ReceiverStream };
		SetUpClientAndServer();
		
		// 2.1 Sender takes authority
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObject })
			.Next([this](FConcertReplication_ChangeAuthority_Response&& Response)
			{
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestTrue(TEXT("Sender > Success taking authority"), Response.RejectedObjects.IsEmpty());
			});

		// 2.2 Sender removes the object
		FConcertReplication_ChangeStream_Request RemoveObjectRequest;
		RemoveObjectRequest.ObjectsToRemove.Add({ SenderStreamId, TestObject });
		ChangeStreamForSenderClientAndValidate(TEXT("Remove Object"), RemoveObjectRequest, {});

		// 2.3 Receiver can now take authority since 2.2 implicitly removed authority
		ClientReplicationManager_Receiver->TakeAuthorityOver({ TestObject })
			.Next([this](FConcertReplication_ChangeAuthority_Response&& Response)
			{
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestTrue(TEXT("Receiver > Success taking authority"), Response.RejectedObjects.IsEmpty());
			});
		
		return true;
	}

	/**
     * Tests that client updates its local cache of the server state when RequestAuthorityChange times out.
     */
    IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FChangingStreamTimeoutRetainsServerState, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.ChangeStreamTimeoutRetainsServerState", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
    bool FChangingStreamTimeoutRetainsServerState::RunTest(const FString& Parameters)
    {
    	// 1. Start up client with float property stream
    	UTestReflectionObject* TestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
    	auto[InitialStreamId, InitialStream] = CreateFloatPropertyStream(*TestObject);
    	SenderArgs.Streams = { InitialStream };
    	SetUpClientAndServer();
		ServerSession->SetTestFlags(EServerSessionTestingFlags::AllowRequestTimeouts);

    	// 2. Timeout request
    	ServerSession->UnregisterCustomRequestHandler<FConcertReplication_ChangeStream_Request>();
    	FConcertReplication_ChangeStream_Request Request;
    	Request.ObjectsToRemove.Add({ InitialStreamId, TestObject });
    	ClientReplicationManager_Sender->ChangeStream(Request);

    	// 3. Check that the local client's server prediction was reverted
    	TArray<FConcertReplicationStream> Streams = ClientReplicationManager_Sender->GetRegisteredStreams();
    	if (Streams.IsEmpty())
    	{
    		AddError(TEXT("Stream change was not reverted"));
    		return true;
    	}
    	TestTrue(TEXT("Timed out Stream change reverted"), Streams[0].BaseDescription.ReplicationMap.ReplicatedObjects.Contains(TestObject));
    	
    	return true;
    }
}
