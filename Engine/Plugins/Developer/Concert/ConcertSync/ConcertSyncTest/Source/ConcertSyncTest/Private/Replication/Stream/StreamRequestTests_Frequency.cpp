// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Data/ReplicationStream.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Util/ChangeStreamsTestBase.h"
#include "Replication/Util/SendReceiveObjectTestBase.h"
#include "Replication/TestReflectionObject.h"

#include "Misc/AutomationTest.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::ConcertSyncTests::Replication::Stream
{
	/**
	 * Tests that frequency settings
	 * - can be changed using all possible options (put, add, remove)
	 * - the operation order is put, then add, then remove
	 * - querying frequency info works
	 * - replication rate cannot be set to 0
	 * - atomicity: failing to change replication frequency fails all other changes
	 * - frequency can only be overriden if object is in object replication map
	 * - frequency override is removed if object is removed from object replication map
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FChangeFrequency, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.Frequency.ChangeFrequency", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FChangeFrequency::RunTest(const FString& Parameters)
	{
		// 1. Start up client with float property stream
		UTestReflectionObject* MainTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		UTestReflectionObject* SecondaryTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		FGuid FloatStreamId;
		FConcertReplicationStream FloatStream;
		Tie(FloatStreamId, FloatStream) = CreateFloatPropertyStream(*MainTestObject);
		SenderArgs.Streams = { FloatStream };
		
		SetUpClientAndServer();
		// We'll intentionally be causing errors below that cause a warning to be logged.
		AddExpectedError(TEXT("Rejecting ChangeStream"), EAutomationExpectedErrorFlags::Contains, 2);

		FConcertBaseStreamInfo ExpectedStream = FloatStream.BaseDescription;
		
		// Change defaults to be SpecifiedRate at 42 update rate.
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange { .NewDefaults = { EConcertObjectReplicationMode::SpecifiedRate, 42 }, .Flags = EConcertReplicationChangeFrequencyFlags::SetDefaults };
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ExpectedStream.FrequencySettings.Defaults = { EConcertObjectReplicationMode::SpecifiedRate, 42 };
			ChangeStreamForSenderClientAndValidate(TEXT("SpecifiedRate with 42"), ChangeRequest, { ExpectedStream });
		}
		
		// Not specifying update flags does not update default replication state.
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange { .NewDefaults = { EConcertObjectReplicationMode::SpecifiedRate, 1 } };
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			ChangeStreamForSenderClientAndValidate(TEXT("Do not update defaults without update flags"), ChangeRequest, { ExpectedStream });
		}
		// Cannot set SpecifiedRate at 0 update rate.
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange { .NewDefaults = { EConcertObjectReplicationMode::SpecifiedRate, 0 }, .Flags = EConcertReplicationChangeFrequencyFlags::SetDefaults };
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			ChangeStreamForSenderClientAndValidate(
				TEXT("Defaults > Rate 0"),
				ChangeRequest,
				{ ExpectedStream },
				EChangeStreamValidation::TestWasHandled | EChangeStreamValidation::TestResponseFailed
				)
				.Next([this, FloatStreamId](FConcertReplication_ChangeStream_Response&& Response)
				{
					const EConcertChangeStreamFrequencyErrorCode* ErrorCode = Response.FrequencyErrors.DefaultFailures.Find(FloatStreamId);
					TestTrue(TEXT("Defaults > Rate 0 > Rejection code"), ErrorCode && *ErrorCode == EConcertChangeStreamFrequencyErrorCode::InvalidReplicationRate);
				});
		}
		
		// Failing frequency update reverts other changes in request (atomicity)
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			
			FConcertReplication_ChangeStream_PutObject PutObject;
			AddFloatProperty(PutObject.Properties);
			PutObject.ClassPath = SecondaryTestObject->GetClass();
			ChangeRequest.ObjectsToPut.Add({ FloatStreamId, SecondaryTestObject }, PutObject);
			
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToAdd.Add(SecondaryTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 0 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ChangeStreamForSenderClientAndValidate(
				TEXT("Failing frequency update reverts other changes in request (atomicity)"),
				ChangeRequest,
				{ ExpectedStream },
				EChangeStreamValidation::TestWasHandled | EChangeStreamValidation::TestResponseFailed
				);
		}
		
		// Add object and change frequency in same request
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			
			FConcertReplication_ChangeStream_PutObject PutObject;
			AddFloatProperty(PutObject.Properties);
			PutObject.ClassPath = SecondaryTestObject->GetClass();
			ChangeRequest.ObjectsToPut.Add({ FloatStreamId, SecondaryTestObject }, PutObject);
			
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToAdd.Add(SecondaryTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 101 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);

			FConcertReplicatedObjectInfo& ExpectedSecondary = ExpectedStream.ReplicationMap.ReplicatedObjects.Add(SecondaryTestObject);
			AddFloatProperty(ExpectedSecondary.PropertySelection);
			ExpectedSecondary.ClassPath = SecondaryTestObject->GetClass();
			ExpectedStream.FrequencySettings.ObjectOverrides.Add(SecondaryTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 101 });
			ChangeStreamForSenderClientAndValidate(TEXT("Register object and set frequency"), ChangeRequest, { ExpectedStream });
		}
		
		return true;
	}

	/**
	 * Tests that requests for unregistered objects rejected:
	 * - Cannot change frequency for unknown stream
	 * - Cannot change frequency for object not registerd to stream
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPutAddAndRemoveFrequencyFlow, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.Frequency.PutAddAndRemoveFrequencyFlow", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FPutAddAndRemoveFrequencyFlow::RunTest(const FString& Parameters)
	{
		UTestReflectionObject* MainTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		UTestReflectionObject* SecondaryTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		auto[FloatStreamId, FloatStream] = CreateFloatPropertyStream(*MainTestObject);
		SenderArgs.Streams = { FloatStream };
		SetUpClientAndServer();

		FConcertBaseStreamInfo ExpectedStream = FloatStream.BaseDescription;
		// Put SpecifiedRate 21
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToPut.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 21 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ExpectedStream.FrequencySettings.ObjectOverrides.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 21 });
			ChangeStreamForSenderClientAndValidate(TEXT("Put SpecifiedRate 21"), ChangeRequest, { ExpectedStream });
		}
		// Add SpecifiedRate 22
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToAdd.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 22 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ExpectedStream.FrequencySettings.ObjectOverrides.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 22 });
			ChangeStreamForSenderClientAndValidate(TEXT("Add SpecifiedRate 22"), ChangeRequest, { ExpectedStream });
		}
		// Remove 
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToRemove.Add(MainTestObject);
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ExpectedStream.FrequencySettings.ObjectOverrides.Remove(MainTestObject);
			ChangeStreamForSenderClientAndValidate(TEXT("Remove"), ChangeRequest, { ExpectedStream });
		}
		
		return true;
	}

	/**
	 * Tests that requests for unregistered objects rejected:
	 * - Cannot change frequency for unknown stream
	 * - Cannot change frequency for object not registerd to stream
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FRemovingObjectRemovesFrequencyEntry, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.Frequency.RemovingObjectRemovesFrequencyEntry", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FRemovingObjectRemovesFrequencyEntry::RunTest(const FString& Parameters)
	{
		// 1. Start up client with float property stream
		UTestReflectionObject* MainTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		UTestReflectionObject* SecondaryTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		auto[FloatStreamId, FloatStream] = CreateFloatPropertyStream(*MainTestObject);
		SenderArgs.Streams = { FloatStream };
		
		SetUpClientAndServer();
		FConcertBaseStreamInfo ExpectedStream = FloatStream.BaseDescription;
		
		// Register SecondaryObject with stream
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			
			FConcertReplication_ChangeStream_PutObject PutObject;
			AddFloatProperty(PutObject.Properties);
			PutObject.ClassPath = SecondaryTestObject->GetClass();
			ChangeRequest.ObjectsToPut.Add({ FloatStreamId, SecondaryTestObject }, PutObject);

			ExpectedStream.ReplicationMap.ReplicatedObjects.Add(SecondaryTestObject, FConcertReplicatedObjectInfo{ PutObject.ClassPath, PutObject.Properties });
			ChangeStreamForSenderClientAndValidate(TEXT("Add MainTestObject & SecondaryObject"), ChangeRequest, { ExpectedStream });
		}
		// Put frequency for MainTestObject & SecondaryObject
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToPut.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 21 });
			FrequencyChange.OverridesToPut.Add(SecondaryTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 22 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ExpectedStream.FrequencySettings.ObjectOverrides.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 21 });
			ExpectedStream.FrequencySettings.ObjectOverrides.Add(SecondaryTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 22 });
			ChangeStreamForSenderClientAndValidate(TEXT("Put SpecifiedRate 21 & 22"), ChangeRequest, { ExpectedStream });
		}
		// Removing MainTestObject removes replication frequency entry
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			ChangeRequest.ObjectsToRemove.Add({ FloatStreamId, MainTestObject });

			ExpectedStream.ReplicationMap.ReplicatedObjects.Remove(MainTestObject);
			ExpectedStream.FrequencySettings.ObjectOverrides.Remove(MainTestObject);
			ChangeStreamForSenderClientAndValidate(TEXT("Unregister object > removes frequency entry"), ChangeRequest, { ExpectedStream });
		}

		return true;
	}

	/**
	 * Tests that requests for unregistered objects rejected:
	 * - Cannot change frequency for unknown stream
	 * - Cannot change frequency for object not registerd to stream
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPutReplacesAll, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.Frequency.PutReplacesAll", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FPutReplacesAll::RunTest(const FString& Parameters)
	{
		// 1. Start up client with float property stream
		UTestReflectionObject* MainTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		UTestReflectionObject* SecondaryTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		auto[FloatStreamId, FloatStream] = CreateFloatPropertyStream(*MainTestObject);
		SenderArgs.Streams = { FloatStream };
		SetUpClientAndServer();
		
		FConcertBaseStreamInfo ExpectedStream = FloatStream.BaseDescription;
		// Register SecondaryObject with stream
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			
			FConcertReplication_ChangeStream_PutObject PutObject;
			AddFloatProperty(PutObject.Properties);
			PutObject.ClassPath = SecondaryTestObject->GetClass();
			ChangeRequest.ObjectsToPut.Add({ FloatStreamId, SecondaryTestObject }, PutObject);

			ExpectedStream.ReplicationMap.ReplicatedObjects.Add(SecondaryTestObject, FConcertReplicatedObjectInfo{ PutObject.ClassPath, PutObject.Properties });
			ChangeStreamForSenderClientAndValidate(TEXT("Add MainTestObject & SecondaryObject"), ChangeRequest, { ExpectedStream });
		}
		

		// Put MainTestObject & SecondaryObject
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToPut.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 10 });
			FrequencyChange.OverridesToPut.Add(SecondaryTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 20 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ExpectedStream.FrequencySettings.ObjectOverrides.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 10 });
			ExpectedStream.FrequencySettings.ObjectOverrides.Add(SecondaryTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 20 });
			ChangeStreamForSenderClientAndValidate(TEXT("Add MainTestObject & SecondaryObject"), ChangeRequest, { ExpectedStream });
		}
		
		// Put replaces all
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToPut.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 100 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);

			ExpectedStream.FrequencySettings.ObjectOverrides.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 100 });
			ExpectedStream.FrequencySettings.ObjectOverrides.Remove(SecondaryTestObject);
			ChangeStreamForSenderClientAndValidate(TEXT("Put replaces all overrides"), ChangeRequest, { ExpectedStream });
		}
		
		return true;
	}

	/**
	 * Tests that requests for unregistered objects rejected:
	 * - Cannot change frequency for unknown stream
	 * - Cannot change frequency for object not registerd to stream
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAddAfterPut, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.Frequency.AddAfterPut", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FAddAfterPut::RunTest(const FString& Parameters)
	{
		// 1. Start up client with float property stream
		UTestReflectionObject* MainTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		auto[FloatStreamId, FloatStream] = CreateFloatPropertyStream(*MainTestObject);
		SenderArgs.Streams = { FloatStream };
		SetUpClientAndServer();

		FConcertBaseStreamInfo ExpectedStream = FloatStream.BaseDescription;
		// Put SpecifiedRate 25, then add 50
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToPut.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 25 });
			FrequencyChange.OverridesToAdd.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 50 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ExpectedStream.FrequencySettings.ObjectOverrides.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 50 });
			ChangeStreamForSenderClientAndValidate(TEXT("Put SpecifiedRate 25, then add 50"), ChangeRequest, { ExpectedStream });
		}
		
		return true;
	}

	/**
	 * Tests that requests for unregistered objects rejected:
	 * - Cannot change frequency for unknown stream
	 * - Cannot change frequency for object not registerd to stream
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FCannotChangeFrequencyOfUnregistered, FChangeStreamsTestBase, "VirtualProduction.Concert.Replication.Stream.Frequency.FailWithUnregisteredStreamOrObject", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FCannotChangeFrequencyOfUnregistered::RunTest(const FString& Parameters)
	{
		// 1. Start up client with float property stream
		UTestReflectionObject* MainTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		UTestReflectionObject* SecondaryTestObject = NewObject<UTestReflectionObject>(GetTransientPackage());
		FGuid FloatStreamId;
		FConcertReplicationStream FloatStream;
		Tie(FloatStreamId, FloatStream) = CreateFloatPropertyStream(*MainTestObject);
		SenderArgs.Streams = { FloatStream };
		
		SetUpClientAndServer();
		// We'll intentionally be causing errors below that cause a warning to be logged.
		AddExpectedError(TEXT("Rejecting ChangeStream"),EAutomationExpectedErrorFlags::Contains, 2);

		// Change defaults to be SpecifiedRate at 42 update rate.
		FConcertBaseStreamInfo ExpectedStream = FloatStream.BaseDescription;
		
		// Cannot change frequency of unregistered stream
		{
			const FGuid UnregisteredStreamId = FGuid::NewGuid();
			
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToPut.Add(MainTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 50 });
			ChangeRequest.FrequencyChanges.Add(UnregisteredStreamId, FrequencyChange);

			ChangeStreamForSenderClientAndValidate(
				TEXT("Cannot change frequency of unregistered stream"),
				ChangeRequest,
				{ ExpectedStream },
				EChangeStreamValidation::TestWasHandled | EChangeStreamValidation::TestResponseFailed
				)
				.Next([this, &UnregisteredStreamId, &MainTestObject](FConcertReplication_ChangeStream_Response&& Response)
				{
					const EConcertChangeObjectFrequencyErrorCode* ErrorCode = Response.FrequencyErrors.OverrideFailures.Find({ UnregisteredStreamId, MainTestObject });
					TestTrue(TEXT("Cannot change frequency of unregistered stream > Rejection code (defaults)"), Response.FrequencyErrors.DefaultFailures.IsEmpty());
					TestTrue(TEXT("Cannot change frequency of unregistered stream > Rejection code (object)"), ErrorCode && *ErrorCode == EConcertChangeObjectFrequencyErrorCode::NotRegistered);
				});
		}

		// Cannot add object that is not registered
		{
			FConcertReplication_ChangeStream_Request ChangeRequest;
			FConcertReplication_ChangeStream_Frequency FrequencyChange;
			FrequencyChange.OverridesToPut.Add(SecondaryTestObject, { EConcertObjectReplicationMode::SpecifiedRate, 100 });
			ChangeRequest.FrequencyChanges.Add(FloatStreamId, FrequencyChange);
			
			ChangeStreamForSenderClientAndValidate(
				TEXT("Cannot add non-registered object"),
				ChangeRequest,
				{ ExpectedStream },
				EChangeStreamValidation::TestWasHandled | EChangeStreamValidation::TestResponseFailed
				)
				.Next([this, FloatStreamId, SecondaryTestObject](FConcertReplication_ChangeStream_Response&& Response)
				{
					const EConcertChangeObjectFrequencyErrorCode* ErrorCode = Response.FrequencyErrors.OverrideFailures.Find({ FloatStreamId, SecondaryTestObject });
					TestTrue(TEXT("Cannot add non-registered object > Rejection code (defaults)"), Response.FrequencyErrors.DefaultFailures.IsEmpty());
					TestTrue(TEXT("Cannot add non-registered object > Rejection code (object)"), ErrorCode && *ErrorCode == EConcertChangeObjectFrequencyErrorCode::NotRegistered);
				});
		}

		return true;
	}
}