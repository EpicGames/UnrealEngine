// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/ClientServerCommunicationTest.h"

#include "Replication/Data/ReplicationStream.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Util/ChangeStreamsTestBase.h"
#include "Replication/Util/SendReceiveObjectTestBase.h"
#include "Replication/TestReflectionObject.h"

#include "Misc/AutomationTest.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::ConcertSyncTests::Replication::Stream
{
	/**
	 * Let's receiving client query the server what the sending client is sending. Checks
	 * - the reported streams,
	 * - that the authority updates,
	 * - the EConcertQueryClientStreamFlags flags
	 *
	 * @note The other tests in StreamRequestTests_{x}.cpp use QueryStream under the hood so all this test does is checking that the skip flags work correctly. 
	 */
	IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FQueryOtherClientStreams, FSendReceiveObjectTestBase, "VirtualProduction.Concert.Replication.Stream.QueryClientStream", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FQueryOtherClientStreams::RunTest(const FString& Parameters)
	{
		// 1. Init
		SetUpClientAndServer();
		
		const FGuid SenderEndpointId = Client_Sender->ClientSessionMock->GetSessionClientEndpointId();
		const FSoftObjectPath TestObjectPath{ TestObject };
		FConcertReplication_QueryReplicationInfo_Request Request;
		Request.ClientEndpointIds = { SenderEndpointId };
		auto TestReplicationMapContent = [this, &SenderEndpointId](const FConcertReplication_QueryReplicationInfo_Response& Response)
		{
			TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			
			const FConcertQueriedClientInfo* Info = Response.ClientInfo.Find(SenderEndpointId);
			if (!Info)
			{
				AddError(TEXT("No info about sending client!"));
				return;
			}
				
			TestEqual(TEXT("Only received info about requested client"), Response.ClientInfo.Num(), 1);
			TestEqual(TEXT("Exactly one stream data"), Info->Streams.Num(), 1);
			if (Info->Streams.IsEmpty())
			{
				return;
			}
				
			const FConcertBaseStreamInfo& StreamDescription = Info->Streams[0];
			TestEqual(TEXT("StreamDescription->Identifier correct"), StreamDescription.Identifier, SenderStreamId);
			TestEqual(TEXT("StreamDescription->ReplicationMap has exactly 1 object"), StreamDescription.ReplicationMap.ReplicatedObjects.Num(), 1);

			const FConcertObjectReplicationMap ExpectedReplicationMap = CreateSenderArgs().Streams[0].BaseDescription.ReplicationMap;
			TestEqual(TEXT("Registered and reported replication maps match"), StreamDescription.ReplicationMap, ExpectedReplicationMap);
		};
		
		// 2.1 Request before taking authority
		bool bReceivedFirstQueryResponse = false;
		ClientReplicationManager_Receiver->QueryClientInfo(Request)
			.Next([this, &SenderEndpointId, &TestReplicationMapContent, &bReceivedFirstQueryResponse](FConcertReplication_QueryReplicationInfo_Response&& Response) mutable
			{
				bReceivedFirstQueryResponse = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				
				TestReplicationMapContent(Response);
				if (const FConcertQueriedClientInfo* Info = Response.ClientInfo.Find(SenderEndpointId))
				{
					TestEqual(TEXT("Contains no authority data"), Info->Authority.Num(), 0);
				}
			});
		TestTrue(TEXT("Received query response (before taking authority)"), bReceivedFirstQueryResponse);

		// 2.2 Request after taking authority
		bool bSenderReceivedResponse = false;
		ClientReplicationManager_Sender->TakeAuthorityOver({ TestObjectPath })
			.Next([this, &bSenderReceivedResponse](const FConcertReplication_ChangeAuthority_Response& Response) mutable
			{
				bSenderReceivedResponse = true;
				TestEqual(TEXT("No rejection taking authority"), Response.RejectedObjects.Num(), 0); 
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
			});
		TestTrue(TEXT("Received response taking authority"), bSenderReceivedResponse);

		bool bReceivedSecondQueryResponse = false;
		ClientReplicationManager_Receiver->QueryClientInfo(Request)
			.Next([this, &SenderEndpointId, &TestReplicationMapContent, &TestObjectPath, &bReceivedSecondQueryResponse](FConcertReplication_QueryReplicationInfo_Response&& Response) mutable
			{
				bReceivedSecondQueryResponse = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				TestReplicationMapContent(Response);
				
				const FConcertQueriedClientInfo* Info = Response.ClientInfo.Find(SenderEndpointId);
				if (!Info || Info->Authority.IsEmpty())
				{
					AddError(TEXT("Missing authority data"));
					return;
				}

				TestEqual(TEXT("Exactly 1 authority stream"), Info->Authority.Num(), 1);
				const FConcertAuthorityClientInfo& AuthorityInfo = Info->Authority[0];
				TestEqual(TEXT("Authority stream ID matches registered stream ID"), AuthorityInfo.StreamId, SenderStreamId);
				TestEqual(TEXT("Has authority over exactly 1 object"), AuthorityInfo.AuthoredObjects.Num(), 1);
				TestTrue(TEXT("Has authority over registered object"), AuthorityInfo.AuthoredObjects.Contains(TestObjectPath));
			});
		TestTrue(TEXT("Received response to taking authority"), bSenderReceivedResponse);
		TestTrue(TEXT("Received query response (after taking authority)"), bReceivedSecondQueryResponse);

		// 2.3 SkipStreamInfo
		bool bReceivedResponse_SkipStreamInfo = false;
		Request.QueryFlags = EConcertQueryClientStreamFlags::SkipStreamInfo;
		ClientReplicationManager_Receiver->QueryClientInfo(Request)
			.Next([this, &SenderEndpointId, &bReceivedResponse_SkipStreamInfo](FConcertReplication_QueryReplicationInfo_Response&& Response) mutable
			{
				bReceivedResponse_SkipStreamInfo = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				if (const FConcertQueriedClientInfo* Info = Response.ClientInfo.Find(SenderEndpointId))
				{
					TestEqual(TEXT("SkipStreamInfo > No stream data"), Info->Streams.Num(), 0);
				}
			});
		TestTrue(TEXT("bReceivedResponse_SkipStreamInfo"), bReceivedResponse_SkipStreamInfo);
		
		// 2.4 SkipProperties
		bool bReceivedResponse_SkipProperties = false;
		Request.QueryFlags = EConcertQueryClientStreamFlags::SkipProperties;
		ClientReplicationManager_Receiver->QueryClientInfo(Request)
			.Next([this, &SenderEndpointId, &TestObjectPath, &bReceivedResponse_SkipProperties](FConcertReplication_QueryReplicationInfo_Response&& Response) mutable
			{
				bReceivedResponse_SkipProperties = true;
				const FConcertQueriedClientInfo* Info = Response.ClientInfo.Find(SenderEndpointId);
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				if (!Info || Info->Streams.IsEmpty())
				{
					AddError(TEXT("SkipProperties > No stream info"));
					return;
				}

				const FConcertReplicatedObjectInfo* ObjectInfo = Info->Streams[0].ReplicationMap.ReplicatedObjects.Find(TestObjectPath);
				if (!ObjectInfo)
				{
					AddError(TEXT("SkipProperties > No object info"));
					return;
				}
				TestEqual(TEXT("SkipProperties > No property data"), ObjectInfo->PropertySelection.ReplicatedProperties.Num(), 0);
			});
		TestTrue(TEXT("bReceivedResponse_SkipProperties"), bReceivedResponse_SkipProperties);
		
		// 2.5 SkipAuthority
		bool bReceivedResponse_SkipAuthority = false;
		Request.QueryFlags = EConcertQueryClientStreamFlags::SkipAuthority;
		ClientReplicationManager_Receiver->QueryClientInfo(Request)
			.Next([this, &SenderEndpointId, &bReceivedResponse_SkipAuthority](FConcertReplication_QueryReplicationInfo_Response&& Response) mutable
			{
				bReceivedResponse_SkipAuthority = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				if (const FConcertQueriedClientInfo* Info = Response.ClientInfo.Find(SenderEndpointId))
				{
					TestEqual(TEXT("SkipAuthority > No authority data"), Info->Authority.Num(), 0);
				}
			});
		TestTrue(TEXT("bReceivedResponse_SkipAuthority"), bReceivedResponse_SkipAuthority);

		// 2.6 SkipFrequency
		FConcertReplication_ChangeStream_Request AddFrequencyInfo;
		FConcertReplication_ChangeStream_Frequency& FrequencyChange = AddFrequencyInfo.FrequencyChanges.Add(SenderStreamId);
		FrequencyChange.OverridesToPut.Add(TestObjectPath) = { EConcertObjectReplicationMode::SpecifiedRate, 20 }; 
		FrequencyChange.NewDefaults = { EConcertObjectReplicationMode::SpecifiedRate, 42 };
		FrequencyChange.Flags = EConcertReplicationChangeFrequencyFlags::SetDefaults;
		// Important: Change Sender here, not Receiver! Receiver is the one querying for Sender.
		ClientReplicationManager_Sender->ChangeStream(AddFrequencyInfo)
			.Next([this](FConcertReplication_ChangeStream_Response&& Response)
			{
				TestTrue(TEXT("Changed frequency"), Response.IsSuccess());
			});
		
		bool bReceivedResponse_SkipFrequency = false;
		Request.QueryFlags = EConcertQueryClientStreamFlags::SkipFrequency;
		ClientReplicationManager_Receiver->QueryClientInfo(Request)
			.Next([this, &SenderEndpointId, &bReceivedResponse_SkipFrequency](FConcertReplication_QueryReplicationInfo_Response&& Response) mutable
			{
				bReceivedResponse_SkipFrequency = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				if (const FConcertQueriedClientInfo* Info = Response.ClientInfo.Find(SenderEndpointId)
					; Info && ensureAlways(Info->Streams.Num() == 1))
				{
					const FConcertStreamFrequencySettings& FrequencySettings = Info->Streams[0].FrequencySettings;
					TestEqual(TEXT("SkipFrequency > No frequency override data"), FrequencySettings.ObjectOverrides.Num(), 0);
				}
			});
		TestTrue(TEXT("bReceivedResponse_SkipFrequency"), bReceivedResponse_SkipFrequency);
		
		return true;
	}
}