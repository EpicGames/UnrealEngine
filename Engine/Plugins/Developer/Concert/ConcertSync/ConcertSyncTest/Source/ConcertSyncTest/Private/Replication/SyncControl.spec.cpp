// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Replication/SyncControlState.h"
#include "Replication/Messages/SyncControl.h"
#include "Util/ConcertClientReplicationBridgeMock.h"
#include "Util/Spec/ObjectTestReplicator.h"
#include "Util/Spec/ReplicationClient.h"
#include "Util/Spec/ReplicationServer.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	BEGIN_DEFINE_SPEC(FReplicationSyncControlSpec, "VirtualProduction.Concert.Replication.SyncControl", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	
	TSharedPtr<FReplicationServer> Server;
	FReplicationClient* Sender = nullptr;
	FReplicationClient* Receiver = nullptr;

	FGuid SenderStreamId = FGuid::NewGuid();
	TSharedPtr<FObjectTestReplicator> ObjectReplicator;

	FObjectReplicationContext MakeSenderToReceiverContext() const { return { *Sender, *Server, *Receiver }; }
	FObjectReplicationContext MakeSenderToReceiverContext(FReplicationClient& OtherReceiver) const { return { *Sender, *Server, OtherReceiver }; }
	
	END_DEFINE_SPEC(FReplicationSyncControlSpec);

	/** This tests that clients respect sync control (see FConcertReplication_ChangeSyncControl). */
	void FReplicationSyncControlSpec::Define()
	{
		BeforeEach([this]
		{
			Server = MakeShared<FReplicationServer>(*this);
			Sender = &Server->ConnectClient();
			Receiver = &Server->ConnectClient();

			ObjectReplicator = MakeShared<FObjectTestReplicator>();

			Sender->JoinReplication(ObjectReplicator->CreateSenderArgs(SenderStreamId));
			// Tells client that the object is ready to replicate
			Sender->GetBridgeMock().InjectAvailableObject(*ObjectReplicator->TestObject);
			// Tells server intent to replicate the object
			Sender->GetClientReplicationManager().TakeAuthorityOver({ ObjectReplicator->TestObject });
		});
		AfterEach([this]
		{
			// Test would hold onto this for rest of engine lifetime. Clean up this mini would-be leak.
			Server.Reset();
			ObjectReplicator.Reset();
		});
		
		Describe("Before receiver client joins", [this]()
		{
			It("Client will not attempt to replicate", [this]()
			{
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { SenderStreamId },
					[this](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
					{
						AddError(TEXT("Sending client replicated object even though they did not have permission."));
					});
				ObjectReplicator->TestValuesWereNotReplicated(*this);

				// This will cause the server to send a sync control event
				Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });
				
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { SenderStreamId });
				ObjectReplicator->TestValuesWereReplicated(*this);
			});
		});

		Describe("When receiver client joins", [this]()
		{
			It("Server grants sync control", [this]()
			{
				bool bReceivedControlEvent = false;
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(
					[this, &bReceivedControlEvent](const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event)
					{
						bReceivedControlEvent = true;
						
						TestEqual(TEXT("Update has 1 object"), Event.NewControlStates.Num(), 1);
						const bool* bMayReplicate = Event.NewControlStates.Find({ SenderStreamId, ObjectReplicator->TestObject });
						TestTrue(TEXT("Is allowed to replicate"), bMayReplicate && *bMayReplicate);
					});
				
				// This will cause the server to send a sync control event
				Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });

				TestTrue(TEXT("Server sent sync control event"), bReceivedControlEvent);
			});
		});
		
		Describe("If server revokes sync control", [this]()
		{
			It("Client will not attempt to replicate", [this]()
			{
				// Like above, this will cause the server to send a sync control event...
				Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });
				
				// ... but we'll just simulate that the server revoked control for any reason
				FConcertReplication_ChangeSyncControl RevokeControl;
				RevokeControl.NewControlStates.Add(FConcertObjectInStreamID{ SenderStreamId, ObjectReplicator->TestObject }, false);
				Server->GetServerSessionMock()->SendCustomEvent(RevokeControl, Sender->GetClientSessionMock()->GetSessionClientEndpointId(), EConcertMessageFlags::ReliableOrdered);
				
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { SenderStreamId },
					[this](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
					{
						AddError(TEXT("Server: Sending client replicated object even though they did not have permission."));
					},
					[this](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
					{
						// This error should not fire without the above error (if it does there's a bug in our testing framework)
						AddError(TEXT("Receiver: Sending client replicated object even though they did not have permission."));
					});
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});
		});

		Describe("When receiver client leaves", [this]()
		{
			// Joining the 2nd client will cause the server to give the sender client sync control
			BeforeEach([this]() { Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject }); });
			
			It("Client will stop replicating", [this]()
			{
				// Sanity test that sending works
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { SenderStreamId });
				ObjectReplicator->TestValuesWereReplicated(*this);

				Receiver->GetClientReplicationManager().LeaveReplicationSession();
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { SenderStreamId });
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});

			It("Server revokes sync control from 1st client", [this]()
			{
				bool bReceivedControlEvent = false;
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(
					[this, &bReceivedControlEvent](const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event)
					{
						bReceivedControlEvent = true;
						
						TestEqual(TEXT("Update has 1 object"), Event.NewControlStates.Num(), 1);
						const bool* bMayReplicate = Event.NewControlStates.Find({ SenderStreamId, ObjectReplicator->TestObject });
						TestTrue(TEXT("Is not allowed to replicate"), bMayReplicate && !*bMayReplicate);
					});

				// And now we check that the 2nd sync control event is received.
				Receiver->GetClientReplicationManager().LeaveReplicationSession();
				TestTrue(TEXT("Server sent sync control event"), bReceivedControlEvent);
			});
		});
		
		Describe("When there are 3 clients",[this]()
		{
			It("Sync control is granted only on first join and revoked only on last leave.", [this]()
			{
				FReplicationClient& ThirdClient = Server->ConnectClient();

				bool bReceivedControlEvent = false;
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(
					[this, &bReceivedControlEvent](const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event)
					{
						bReceivedControlEvent = true;
					});

				// 2nd client joining sends the event
				Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });
				TestTrue(TEXT("2nd client joining causes sync control to be sent"), bReceivedControlEvent);

				// 3rd client joining does not send the event because the sender is now sending to clients 2 & 3
				bReceivedControlEvent = false;
				ThirdClient.JoinReplicationAsListener({ ObjectReplicator->TestObject });
				TestFalse(TEXT("3rd receiver joining does not cause any sync control to be sent"), bReceivedControlEvent);

				// 2nd client leaving does not send because the sender is still allowed to send to client 3
				bReceivedControlEvent = false;
				Receiver->GetClientReplicationManager().LeaveReplicationSession();
				TestFalse(TEXT("2nd receiver leaving does not cause any sync control to be sent"), bReceivedControlEvent);

				// 3rd client leaving does cause the event to be sent
				bReceivedControlEvent = false;
				ThirdClient.GetClientReplicationManager().LeaveReplicationSession();
				TestTrue(TEXT("3rd receiver causes a sync control to be sent"), bReceivedControlEvent);
			});

			It("Replication continues to work when only 1 client leaves", [this]()
			{
				FReplicationClient& ThirdClient = Server->ConnectClient();
				
				Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });
				ThirdClient.JoinReplicationAsListener({ ObjectReplicator->TestObject });

				// This should not send any sync control event because the 3rd client is still there (tested in previous case) ...
				Receiver->GetClientReplicationManager().LeaveReplicationSession();

				// ... hence replication still works, too.
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(ThirdClient), { SenderStreamId });
				ObjectReplicator->TestValuesWereReplicated(*this);
			});
		});

		Describe("When sender changes authority", [this]()
		{
			BeforeEach([this]()
			{
				// Joining the 2nd client will cause the server to give the sender client sync control
				Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });

				// No events are sent for authority changes, see FConcertReplication_ChangeSyncControl documentation.
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(
					[this](const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event)
					{
						AddError(TEXT("No sync event was supposed to be sent."));
					});
			});
			
			It("Client loses sync control when giving up authority explicitly", [this]()
			{
				Sender->GetClientReplicationManager().ReleaseAuthorityOf({ ObjectReplicator->TestObject })
					.Next([this](FConcertReplication_ChangeAuthority_Response&& Response)
					{
						// Server did not include the sync loss because the change is implicit (documentation).
						TestTrue(TEXT("SyncControl.IsEmpty()"), Response.SyncControl.IsEmpty());
					});
				
				// This tests that the sender no longer has sync control
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { SenderStreamId });
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});

			It("Client loses sync control when giving up authority implicitly", [this]()
			{
				FConcertReplication_ChangeStream_Request StreamChange;
				StreamChange.ObjectsToRemove.Add({ SenderStreamId, ObjectReplicator->TestObject });
				Sender->GetClientReplicationManager().ChangeStream(StreamChange);
				
				// This tests that the sender no longer has sync control
				ObjectReplicator->SimulateSendObjectToReceiver(*this, MakeSenderToReceiverContext(), { SenderStreamId });
				ObjectReplicator->TestValuesWereNotReplicated(*this);
			});

			It("Client gets sync control again when taking back authority", [this]()
			{
				Sender->GetClientReplicationManager().ReleaseAuthorityOf({ ObjectReplicator->TestObject });
				Sender->GetClientReplicationManager().TakeAuthorityOver({ ObjectReplicator->TestObject })
					.Next([this](FConcertReplication_ChangeAuthority_Response&& Response)
					{
						// The server should instantly give sync control because there is another client
						TestEqual(TEXT("SyncControl.Num() == 1"), Response.SyncControl.NewControlStates.Num(), 1);
						
						const FConcertObjectInStreamID ObjectId { SenderStreamId, ObjectReplicator->TestObject };
						const bool* bNewState =Response.SyncControl.NewControlStates.Find(ObjectId);
						TestTrue(TEXT("TestObject is enabled again"), bNewState && *bNewState);
					});
			});
		});

		// At this point, everything should work server-side. Now test client-side prediction.
		Describe("Local client sync control cache", [this]()
		{
			// Joining the 2nd client will cause the server to give the sender client sync control
			BeforeEach([this]()
			{
				Receiver->JoinReplicationAsListener({ ObjectReplicator->TestObject });

				// No events are sent for joins nor authority changes, see FConcertReplication_ChangeSyncControl documentation.
				Sender->GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(
					[this](const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event)
					{
						AddError(TEXT("Server sent sync control event."));
					});
			});
			
			It("Is correct after join", [this]
			{
				IConcertClientReplicationManager& Manager = Sender->GetClientReplicationManager();
				TestEqual("NumSyncControlledObjects() == 1", Manager.NumSyncControlledObjects(), 1);
				TestTrue("HasSyncControl()", Manager.HasSyncControl({ SenderStreamId, ObjectReplicator->TestObject }));
			});

			It("Is correct after explicit authority change", [this]()
			{
				IConcertClientReplicationManager& Manager = Sender->GetClientReplicationManager();
				
				// First release authority over the object and test that the client's local sync control cache updates correctly.
				ConcertSyncCore::Replication::FSyncControlState SyncControlBeforeRelease = Manager.GetSyncControlledObjects();
				FConcertReplication_ChangeAuthority_Request ReleaseAuthority { .ReleaseAuthority = { { ObjectReplicator->TestObject, {{ SenderStreamId }} } } };
				Manager.RequestAuthorityChange(ReleaseAuthority)
					.Next([this, &Manager, &ReleaseAuthority, &SyncControlBeforeRelease](FConcertReplication_ChangeAuthority_Response&& Response)
					{
						// Aggregate is validated in SyncControlState.spec.cpp
						SyncControlBeforeRelease.AppendAuthorityChange(ReleaseAuthority, Response.SyncControl);
						TestTrue(TEXT("Client predicted correctly"), SyncControlBeforeRelease == Manager.GetSyncControlledObjects());
					});

				// Now take back authority and test that the client's local sync control cache updates correctly.
				ConcertSyncCore::Replication::FSyncControlState SyncControlBeforeTake = Manager.GetSyncControlledObjects();
				FConcertReplication_ChangeAuthority_Request TakeAuthority { .TakeAuthority = { { ObjectReplicator->TestObject, {{ SenderStreamId }} } } };
				Manager.RequestAuthorityChange(ReleaseAuthority)
					.Next([this, &Manager, &SyncControlBeforeTake, &TakeAuthority](FConcertReplication_ChangeAuthority_Response&& Response)
					{
						// Aggregate is validated in SyncControlState.spec.cpp
						SyncControlBeforeTake.AppendAuthorityChange(TakeAuthority, Response.SyncControl);
						TestTrue(TEXT("Client predicted correctly"), SyncControlBeforeTake == Manager.GetSyncControlledObjects());
					});
			});

			It("Is correct after implicit authority change", [this]()
			{
				IConcertClientReplicationManager& Manager = Sender->GetClientReplicationManager();
				
				// Removing the object will implicitly get rid of the object's authority and sync control
				ConcertSyncCore::Replication::FSyncControlState SyncControlBeforeRelease = Manager.GetSyncControlledObjects();
				FConcertReplication_ChangeStream_Request StreamChange;
				StreamChange.ObjectsToRemove.Add({ SenderStreamId, ObjectReplicator->TestObject });
				Manager.ChangeStream(StreamChange)
					.Next([this, &Manager, &StreamChange, &SyncControlBeforeRelease](FConcertReplication_ChangeStream_Response&& Response)
					{
						// Aggregate is validated in SyncControlState.spec.cpp
						SyncControlBeforeRelease.AppendStreamChange(StreamChange);
						TestTrue(TEXT("Client predicted correctly"), SyncControlBeforeRelease == Manager.GetSyncControlledObjects());
					});
			});
		});
	}
}
