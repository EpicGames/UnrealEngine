// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "UObject/Class.h"
#include "TestReflectionObject.generated.h"

USTRUCT()
struct FNativeStruct
{
	GENERATED_BODY()

	UPROPERTY()
	float Float{};
	
	bool Serialize(FArchive& Ar)
	{
		Ar << Float;
		return true;
	}
};

inline uint32 GetTypeHash(const FNativeStruct& Struct){ return GetTypeHash(Struct.Float); }

template<>
struct TStructOpsTypeTraits<FNativeStruct> : public TStructOpsTypeTraitsBase2<FNativeStruct>
{
	enum 
	{
		WithSerializer = true
	};
};

USTRUCT()
struct FTestReplicationStruct
{
	GENERATED_BODY()

	UPROPERTY()
	float Value{}; // The name of this property is purposefully equal to FConcertPropertyChain::InternalContainerPropertyValueName.
	UPROPERTY()
	FVector Vector = FVector::ZeroVector;
	UPROPERTY()
	FNativeStruct NativeStruct;

	UPROPERTY()
	TArray<float> FloatArray;
	UPROPERTY()
	TArray<FString> StringArray;
	UPROPERTY()
	TArray<FNativeStruct> NativeStructArray;

	UPROPERTY()
	TSet<float> FloatSet;
	UPROPERTY()
	TSet<FString> StringSet;
	UPROPERTY()
	TSet<FNativeStruct> NativeStructSet;
	
	UPROPERTY()
	TMap<FString, float> StringToFloat;
	UPROPERTY()
	TMap<FString, FVector> StringToVector;
	UPROPERTY()
	TMap<FString, FNativeStruct> StringToNativeStruct;
};

USTRUCT()
struct FTestNestedReplicationStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FTestReplicationStruct Nested;

	UPROPERTY()
	TArray<FTestReplicationStruct> NestedArray;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTestReflectionDelegate);

UCLASS()
class UTestReflectionObject : public UObject
{
	GENERATED_BODY()
public:
	
	UPROPERTY(EditAnywhere, Category = "Default")
	float Float{};
	
	UPROPERTY()
	FVector Vector = FVector::ZeroVector;

	UPROPERTY()
	FTestNestedReplicationStruct TestStruct;

	// These properties are invalid to replicate

	UPROPERTY(BlueprintAssignable, Category = "Dummy")
	FTestReflectionDelegate DelegateTest;

	UPROPERTY(Instanced)
	TObjectPtr<UObject> InstancedSubobject;
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UObject>> InstancedSubobjectArray;
	UPROPERTY(Instanced)
	TSet<TObjectPtr<UObject>> InstancedSubobjectSet;
	UPROPERTY(Instanced)
	TMap<FName, TObjectPtr<UObject>> InstancedSubobjectMap;
};