// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/TestReflectionObject.h"
#include "Replication/Data/ObjectReplicationMap.h"
#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"

namespace UE::ConcertSyncTests::Replication::UI
{
	BEGIN_DEFINE_SPEC(FReplicationStreamSpec, "VirtualProduction.Concert.Replication.UI", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	FConcertObjectReplicationMap ReplicationMap;
	TSharedPtr<ConcertSharedSlate::IEditableReplicationStreamModel> Model;
	UTestReflectionObject* Object;

	TOptional<FConcertPropertyChain> Prop_TestStruct;
	TOptional<FConcertPropertyChain> Prop_TestStruct_Nested;
	TOptional<FConcertPropertyChain> Prop_TestStruct_Nested_Vector;
	TOptional<FConcertPropertyChain> Prop_TestStruct_Nested_Vector_X;
	TOptional<FConcertPropertyChain> Prop_TestStruct_NestedArray;
	TOptional<FConcertPropertyChain> Prop_TestStruct_NestedArray_Value;
	END_DEFINE_SPEC(FReplicationStreamSpec);

	void FReplicationStreamSpec::Define()
	{
		Object = GetMutableDefault<UTestReflectionObject>();
		Prop_TestStruct
			= FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct") });
		Prop_TestStruct_Nested
			= FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("Nested") });
		Prop_TestStruct_Nested_Vector
			= FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector") });
		Prop_TestStruct_Nested_Vector_X
			= FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("Nested"), TEXT("Vector"), TEXT("X") });
		Prop_TestStruct_NestedArray
			= FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray") });
		Prop_TestStruct_NestedArray_Value
			= FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("TestStruct"), TEXT("NestedArray"), TEXT("Value") });
		
		BeforeEach([this]
		{
			if (!Prop_TestStruct || !Prop_TestStruct_Nested || !Prop_TestStruct_Nested_Vector || !Prop_TestStruct_Nested_Vector_X ||! Prop_TestStruct_NestedArray || !Prop_TestStruct_NestedArray_Value)
			{
				AddError(TEXT("FConcertPropertyChain::CreateFromPath failed"));
			}

			ReplicationMap = FConcertObjectReplicationMap{};
			Model = UE::ConcertSharedSlate::CreateBaseStreamModel(&ReplicationMap);
			Model->AddObjects({ Object });
		});

		Describe("Adding a property", [this]()
		{
			It("Adds all parent properties", [this]()
			{
				Model->AddProperties(Object, { *Prop_TestStruct_Nested_Vector_X });

				TestEqual(TEXT("4 properties added"), Model->GetAllProperties(Object).Num(), 4);
				TestTrue(TEXT("TestStruct"), Model->HasProperty(Object, *Prop_TestStruct));
				TestTrue(TEXT("TestStruct.Nested"), Model->HasProperty(Object, *Prop_TestStruct_Nested));
				TestTrue(TEXT("TestStruct.Nested.Vector"), Model->HasProperty(Object, *Prop_TestStruct_Nested_Vector));
				TestTrue(TEXT("TestStruct.Nested.Vector.X"), Model->HasProperty(Object, *Prop_TestStruct_Nested_Vector_X));
			});
		});
		
		Describe("Removing a property", [this]()
		{
			It("Should remove parent properties left with 0 children", [this]()
			{
				Model->AddProperties(Object, { *Prop_TestStruct_Nested_Vector_X, *Prop_TestStruct_NestedArray_Value });
				Model->RemoveProperties(Object, { *Prop_TestStruct_Nested_Vector_X });
				
				// Remove TestStruct.Nested.Vector and TestStruct.Nested as well but not TestStruct since it still references TestStruct.NestedArray.
				TestEqual(TEXT("3 properties remain"), Model->GetAllProperties(Object).Num(), 3);
				TestTrue(TEXT("TestStruct"), Model->HasProperty(Object, *Prop_TestStruct));
				TestTrue(TEXT("TestStruct.Nested.NestedArray"), Model->HasProperty(Object, *Prop_TestStruct_NestedArray));
				TestTrue(TEXT("TestStruct.Nested.NestedArray.Value"), Model->HasProperty(Object, *Prop_TestStruct_NestedArray_Value));
			});
		});
	}
}
