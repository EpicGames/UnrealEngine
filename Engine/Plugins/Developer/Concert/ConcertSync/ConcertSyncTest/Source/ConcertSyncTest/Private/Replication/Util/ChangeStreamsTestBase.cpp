// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChangeStreamsTestBase.h"

#include "Replication/Data/ReplicationStream.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/TestReflectionObject.h"
#include "Replication/Util/SendReceiveGenericStreamTestBase.h"

#include "Misc/AutomationTest.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::ConcertSyncTests::Replication
{
	FChangeStreamsTestBase::FChangeStreamsTestBase(const FString& InName, const bool bInComplexTask)
			: FSendReceiveGenericTestBase(InName, bInComplexTask)
	{}
	
	void FChangeStreamsTestBase::AddFloatProperties(const UTestReflectionObject& TestObject, FConcertReplicationStream& Stream)
	{
		FConcertPropertySelection Properties;
		AddFloatProperty(Properties);
		const FConcertReplicatedObjectInfo AllProperties { TestObject.GetClass(), MoveTemp(Properties) };
		Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(&TestObject, AllProperties);
	}
	
	TTuple<FGuid, FConcertReplicationStream> FChangeStreamsTestBase::CreateFloatPropertyStream(
		const UTestReflectionObject& TestObject,
		FConcertObjectReplicationSettings DefaultFrequencySettings
		)
	{
		const FGuid NewStreamId = FGuid::NewGuid();

		// Use Realtime replication because otherwise our test will need to pass latent time due to the frequency system
		FConcertReplicationStream SendingStream;
		SendingStream.BaseDescription.Identifier = NewStreamId;
		SendingStream.BaseDescription.FrequencySettings.Defaults = DefaultFrequencySettings;
		AddFloatProperties(TestObject, SendingStream);
		return { NewStreamId, SendingStream };
	}

	TTuple<FGuid, FConcertReplicationStream> FChangeStreamsTestBase::CreateVectorPropertyStream(
		const UTestReflectionObject& TestObject,
		FConcertObjectReplicationSettings DefaultFrequencySettings
		)
	{
		const FGuid NewStreamId = FGuid::NewGuid();
		FConcertPropertySelection Properties;
		AddVectorProperty(Properties);
		
		// Use Realtime replication because otherwise our test will need to pass latent time due to the frequency system
		const FConcertReplicatedObjectInfo AllProperties { TestObject.GetClass(), MoveTemp(Properties) };
		FConcertReplicationStream SendingStream;
		SendingStream.BaseDescription.Identifier = NewStreamId;
		SendingStream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(&TestObject, AllProperties);
		SendingStream.BaseDescription.FrequencySettings.Defaults = DefaultFrequencySettings;
		return { NewStreamId, SendingStream };
	}

	FConcertPropertySelection& FChangeStreamsTestBase::GetPropertySelection(FConcertReplicationStream& Stream, const UTestReflectionObject& TestObject)
	{
		return Stream.BaseDescription.ReplicationMap.ReplicatedObjects.FindOrAdd(&TestObject).PropertySelection;
	}

	void FChangeStreamsTestBase::AddFloatProperty(FConcertPropertySelection& PropertySelection)
	{
		const FProperty* FloatProperty = UTestReflectionObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestReflectionObject, Float));
		const FConcertPropertyChain PropertyChain(nullptr, *FloatProperty);
		PropertySelection.ReplicatedProperties.Add(PropertyChain);
	}

	void FChangeStreamsTestBase::AddVectorProperty(FConcertPropertySelection& PropertySelection)
	{
		PropertySelection.ReplicatedProperties.Add(*FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector") }));
		PropertySelection.ReplicatedProperties.Add(*FConcertPropertyChain::CreateFromPath(*UTestReflectionObject::StaticClass(), { TEXT("Vector"), TEXT("X") }));
	}

	/** Sends a ChangeStream request and validates that ExpectedStreams is equal to the registered streams. */
	TFuture<FConcertReplication_ChangeStream_Response> FChangeStreamsTestBase::ChangeStreamForSenderClientAndValidate(
		const FString& InTestName,
		const FConcertReplication_ChangeStream_Request& Request,
		const TArray<FConcertBaseStreamInfo>& ExpectedStreams,
		EChangeStreamValidation ValidationFlags
		)
	{
		using namespace ConcertSyncClient::Replication;
		
		bool bModifiedInitialStream = false;
		TFuture<FConcertReplication_ChangeStream_Response> Future = ClientReplicationManager_Sender->ChangeStream(Request)
			.Next([this, &InTestName, ValidationFlags, &bModifiedInitialStream](FConcertReplication_ChangeStream_Response&& Response)
			{
				bModifiedInitialStream = true;
				
				const bool bValidatedErrorCode =
					!EnumHasAnyFlags(ValidationFlags, EChangeStreamValidation::TestWasHandled | EChangeStreamValidation::TestWasTimeout)
					|| (EnumHasAnyFlags(ValidationFlags, EChangeStreamValidation::TestWasHandled) && Response.ErrorCode == EReplicationResponseErrorCode::Handled)
					|| (EnumHasAnyFlags(ValidationFlags, EChangeStreamValidation::TestWasTimeout) && Response.ErrorCode == EReplicationResponseErrorCode::Timeout);
				TestTrue(FString::Printf(TEXT("%s: Validate error code"), *InTestName), bValidatedErrorCode);
				
				const bool bResponseValidationResult =
					!EnumHasAnyFlags(ValidationFlags, EChangeStreamValidation::TestResponseSuccess | EChangeStreamValidation::TestResponseFailed)
					|| (EnumHasAnyFlags(ValidationFlags, EChangeStreamValidation::TestResponseSuccess) && Response.IsSuccess())
					|| (EnumHasAnyFlags(ValidationFlags, EChangeStreamValidation::TestResponseFailed) && Response.IsFailure());
				TestTrue(FString::Printf(TEXT("%s: Validate response"), *InTestName), bResponseValidationResult);
				
				return Response;
			});
		TestTrue(FString::Printf(TEXT("%s > Modify Stream > Received response"), *InTestName), bModifiedInitialStream);
		ValidateSenderClientStreams(InTestName, ExpectedStreams);
		
		return MoveTemp(Future);
	}
	
	void FChangeStreamsTestBase::ValidateSenderClientStreams(const FString& InTestName, const TArray<FConcertBaseStreamInfo>& ExpectedStreams)
	{
		using namespace ConcertSyncClient::Replication;

		// Test that remote clients see the receive the same as Streams
		bool bQueriedClientInfo = false;
		const FGuid SenderId = Client_Sender->ClientSessionMock->GetSessionClientEndpointId();
		ClientReplicationManager_Receiver->QueryClientInfo({{ { SenderId } }})
			.Next([this, &InTestName, &bQueriedClientInfo, &ExpectedStreams, &SenderId](FConcertReplication_QueryReplicationInfo_Response&& Response)
			{
				bQueriedClientInfo = true;
				TestTrue(TEXT("ErrorCode == Handled"), Response.ErrorCode == EReplicationResponseErrorCode::Handled);
				const FConcertQueriedClientInfo* Info = Response.ClientInfo.Find(SenderId);
				const bool bRegisteredAndQueriedStreamsAreEqual = Info->Streams == ExpectedStreams;
				TestTrue(FString::Printf(TEXT("%s > Registered and queried streams are equal"), *InTestName), bRegisteredAndQueriedStreamsAreEqual);
			});

		// Validate the local client's cache equals  Streams
		const TArray<FConcertReplicationStream> LocalStreams = ClientReplicationManager_Sender->GetRegisteredStreams();
		TArray<FConcertBaseStreamInfo> TransformedLocalStreams;
		Algo::Transform(LocalStreams, TransformedLocalStreams, [](const FConcertReplicationStream& Stream){ return Stream.BaseDescription; });
		TestEqual(FString::Printf(TEXT("%s > Local streams match registered streams"), *InTestName), TransformedLocalStreams, ExpectedStreams);
	};
}