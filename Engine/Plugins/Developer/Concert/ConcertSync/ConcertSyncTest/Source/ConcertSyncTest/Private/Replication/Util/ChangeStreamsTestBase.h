// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ReplicationStream.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Messages/ChangeStream.h"
#include "SendReceiveGenericStreamTestBase.h"

#include "Async/Future.h"

class UTestReflectionObject;

namespace UE::ConcertSyncTests::Replication
{
	enum class EChangeStreamValidation : uint8
	{
		/** Do special flags */
		None = 0,

		/** Test that the ErrorCode is EReplicationResponseErrorCode::Handled. */
		TestWasHandled = 1 << 0,
		/** Test that the ErrorCode is EReplicationResponseErrorCode::Timeout. */
		TestWasTimeout = 1 << 1,
		
		/** Test that the stream change request succeeded */
		TestResponseSuccess = 1 << 2,
		/** Test that the stream change request  failed */
		TestResponseFailed = 1 << 3,
	};
	ENUM_CLASS_FLAGS(EChangeStreamValidation);
	
	/**
	 * Base test for testing ChangeStream requests.
	 * 
	 * While there is no sending nor receiving of replicated data, subclasses will still refer to the clients as "Sender"
	 * and "Receiver".
	 */
	class FChangeStreamsTestBase : public FSendReceiveGenericTestBase
	{
	public:

		FChangeStreamsTestBase(const FString& InName, const bool bInComplexTask);

	protected:
		
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderArgs;
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs ReceiverArgs;

		//~ Begin FSendReceiveTestBase Interface
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateSenderArgs() override { return SenderArgs; }
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateReceiverArgs() override { return ReceiverArgs; }
		//~ End FSendReceiveTestBase Interface

		static void AddFloatProperties(const UTestReflectionObject& TestObject, FConcertReplicationStream& Stream);
		
		static TTuple<FGuid, FConcertReplicationStream> CreateFloatPropertyStream(
			const UTestReflectionObject& TestObject,
			FConcertObjectReplicationSettings DefaultFrequencySettings = { EConcertObjectReplicationMode::Realtime }
			);
		static TTuple<FGuid, FConcertReplicationStream> CreateVectorPropertyStream(
			const UTestReflectionObject& TestObject,
			FConcertObjectReplicationSettings DefaultFrequencySettings = { EConcertObjectReplicationMode::Realtime }
			);
		
		static FConcertPropertySelection& GetPropertySelection(FConcertReplicationStream& Stream, const UTestReflectionObject& TestObject);

		static void AddFloatProperty(FConcertPropertySelection& PropertySelection);
		static void AddVectorProperty(FConcertPropertySelection& PropertySelection);

		/** Sends a ChangeStream request and validates that ExpectedStreams is equal to the registered streams. */
		TFuture<FConcertReplication_ChangeStream_Response> ChangeStreamForSenderClientAndValidate(
			const FString& InTestName,
			const FConcertReplication_ChangeStream_Request& Request,
			const TArray<FConcertBaseStreamInfo>& ExpectedStreams,
			EChangeStreamValidation ValidationFlags = EChangeStreamValidation::TestWasHandled | EChangeStreamValidation::TestResponseSuccess
			);
		void ValidateSenderClientStreams(const FString& InTestName, const TArray<FConcertBaseStreamInfo>& ExpectedStreams);
	};
}
