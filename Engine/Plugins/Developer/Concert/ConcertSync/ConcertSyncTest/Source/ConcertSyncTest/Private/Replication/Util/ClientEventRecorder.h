// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ConcertSyncTests::Replication
{
	enum class EEventType
	{
		PreStreamChange,
		PostStreamChange,
		PreAuthorityChange,
		PostAuthorityChange,
		PreSyncControlChange,
		PostSyncControlChange,
		PreRemoteEditApplied,
		PostRemoteEditApplied
	};

	/** Records the order in which events are broadcast on an IConcertClientReplicationManager. */
	class FClientEventRecorder : public FNoncopyable
	{
		IConcertClientReplicationManager& ReplicationManager;
		TArray<EEventType> EventOrder;
	public:

		FClientEventRecorder(IConcertClientReplicationManager& ReplicationManager UE_LIFETIMEBOUND)
			: ReplicationManager(ReplicationManager)
		{
			ReplicationManager.OnPreStreamsChanged().AddRaw(this, &FClientEventRecorder::HandleEvent, EEventType::PreStreamChange);
			ReplicationManager.OnPostStreamsChanged().AddRaw(this, &FClientEventRecorder::HandleEvent, EEventType::PostStreamChange);
			ReplicationManager.OnPreAuthorityChanged().AddRaw(this, &FClientEventRecorder::HandleEvent, EEventType::PreAuthorityChange);
			ReplicationManager.OnPostAuthorityChanged().AddRaw(this, &FClientEventRecorder::HandleEvent, EEventType::PostAuthorityChange);
			ReplicationManager.OnPreSyncControlChanged().AddRaw(this, &FClientEventRecorder::HandleEvent, EEventType::PreSyncControlChange);
			ReplicationManager.OnPostSyncControlChanged().AddRaw(this, &FClientEventRecorder::HandleEvent, EEventType::PostSyncControlChange);
			ReplicationManager.OnPreRemoteEditApplied().AddRaw(this, &FClientEventRecorder::HandleEvent, EEventType::PreRemoteEditApplied);
			ReplicationManager.OnPostRemoteEditApplied().AddRaw(this, &FClientEventRecorder::HandleEvent, EEventType::PostRemoteEditApplied);
		}
		~FClientEventRecorder()
		{
			ReplicationManager.OnPreStreamsChanged().RemoveAll(this);
			ReplicationManager.OnPostStreamsChanged().RemoveAll(this);
			ReplicationManager.OnPreAuthorityChanged().RemoveAll(this);
			ReplicationManager.OnPostAuthorityChanged().RemoveAll(this);
			ReplicationManager.OnPreSyncControlChanged().RemoveAll(this);
			ReplicationManager.OnPostSyncControlChanged().RemoveAll(this);
			ReplicationManager.OnPreRemoteEditApplied().RemoveAll(this);
			ReplicationManager.OnPostRemoteEditApplied().RemoveAll(this);
		}

		const TArray<EEventType>& GetEventOrder() const { return EventOrder; }
		void Clear() { EventOrder.Empty(); }

	private:

		void HandleEvent(const EEventType EventType) { EventOrder.Add(EventType); }
		void HandleEvent(const ConcertSyncClient::Replication::FRemoteEditEvent&, const EEventType EventType) { EventOrder.Add(EventType);  }
	};
}
