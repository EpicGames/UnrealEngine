// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/AnyOf.h"
#include "Replication/IConcertClientReplicationBridge.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertSyncTests::Replication
{
	class FConcertClientReplicationBridgeMock : public IConcertClientReplicationBridge
	{
	public:

		TArray<FSoftObjectPath> TrackedObjects;
		TArray<UObject*> AvailableObjects;
		
		FConcertClientReplicationBridgeObjectEvent OnObjectDiscoveredDelegate;
		FConcertClientReplicationBridgeObjectPathEvent OnObjectRemovedDelegate;

		void InjectAvailableObject(UObject& Object)
		{
			AvailableObjects.Add(&Object);
			OnObjectDiscoveredDelegate.Broadcast(Object);
		}
		void HideObject(UObject& Object)
		{
			AvailableObjects.Remove(&Object);
			OnObjectRemovedDelegate.Broadcast(&Object);
		}
		
		virtual void PushTrackedObjects(TArrayView<const FSoftObjectPath> InTrackedObjects) override
		{
			for (const FSoftObjectPath& Path : InTrackedObjects)
			{
				TrackedObjects.Add(Path);

				UObject** AvailableObject = AvailableObjects.FindByPredicate([&Path](UObject* Object)
				{
					return Path == Object;
				});
				if (AvailableObject)
				{
					OnObjectDiscoveredDelegate.Broadcast(**AvailableObject);
				}
			}
		}
		
		virtual void PopTrackedObjects(TArrayView<const FSoftObjectPath> InTrackedObjects) override
		{
			for (const FSoftObjectPath& Path : InTrackedObjects)
			{
				TrackedObjects.RemoveSingle(Path);
			}
		}

		virtual bool IsObjectAvailable(const FSoftObjectPath& Path) override { return Algo::AnyOf(AvailableObjects, [&Path](UObject* Object){ return Path == Object; }); }
		virtual UObject* FindObjectIfAvailable(const FSoftObjectPath& Path) override
		{
			for (UObject* Object : AvailableObjects)
			{
				if (Path == Object)
				{
					return Object;
				}
			}
			return nullptr;
		}

		virtual FConcertClientReplicationBridgeObjectEvent& OnObjectDiscovered() override { return OnObjectDiscoveredDelegate; }
		virtual FConcertClientReplicationBridgeObjectPathEvent& OnObjectHidden() override { return OnObjectRemovedDelegate; }
	};
}
