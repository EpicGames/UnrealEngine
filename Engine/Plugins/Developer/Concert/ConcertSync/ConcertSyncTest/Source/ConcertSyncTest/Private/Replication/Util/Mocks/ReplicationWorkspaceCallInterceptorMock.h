// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IReplicationWorkspace.h"
#include "Replication/Messages/ReplicationActivity.h"

#include "Misc/Optional.h"
#include "Misc/Guid.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"

namespace UE::ConcertSyncTests::Replication
{
	class FReplicationWorkspaceCallInterceptorMock : public ConcertSyncServer::Replication::IReplicationWorkspace
	{
	public:

		/** Arguments of last ProduceClientLeaveReplicationActivity call. */
		TOptional<TTuple<FGuid, FConcertSyncReplicationPayload_LeaveReplication>> LastCall_ProduceClientLeaveReplicationActivity;
		/** Arguments of last ProduceClientMuteReplicationActivity call. */
		TOptional<TTuple<FGuid, FConcertSyncReplicationPayload_Mute>> LastCall_ProduceClientMuteReplicationActivity;
		/** Arguments of last GetLastLeaveReplicationActivityByClient call. */
		mutable TOptional<TTuple<FConcertSessionClientInfo>> LastCall_GetLastReplicationActivityByClient;
		/** Arguments of last GetLastLeaveReplicationActivityByClient call. */
		mutable TOptional<TTuple<int64>> LastCall_GetLeaveReplicationActivityById;

		/** The result to return in ProduceClientLeaveReplicationActivity. */
		TOptional<int64> ReturnResult_ProduceClientLeaveReplicationActivity = 0;
		/** The result to return in ProduceClientLeaveReplicationActivity. */
		TOptional<int64> ReturnResult_ProduceClientMuteReplicationActivity = 0;
		/** The result to return in GetLastReplicationActivityByClient. */
		TMap<EConcertSyncReplicationActivityType, FConcertSyncReplicationActivity> ReturnResult_GetLastReplicationActivityByClient;
		/** The result to return in GetReplicationEventById. */
		TOptional<FConcertSyncReplicationEvent> ReturnResult_GetReplicationEventById;
		/** The values to enumerate in EnumerateMuteActivities. */
		TOptional<TArray<FConcertSyncReplicationActivity>> ReturnResult_EnumerateActivities;
		
		//~ Begin IReplicationWorkspace Interface
		virtual TOptional<int64> ProduceReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationEvent& EventData) override
		{
			switch (EventData.ActivityType)
			{
			case EConcertSyncReplicationActivityType::LeaveReplication:
				{
					FConcertSyncReplicationPayload_LeaveReplication Data;
					const bool bSuccess = EventData.GetPayload(Data);
					ensure(bSuccess);
					
					LastCall_ProduceClientLeaveReplicationActivity = MakeTuple(EndpointId, Data);
					return ReturnResult_ProduceClientLeaveReplicationActivity;
				}
			case EConcertSyncReplicationActivityType::Mute:
				{
					FConcertSyncReplicationPayload_Mute Data;
					const bool bSuccess = EventData.GetPayload(Data);
					ensure(bSuccess);
					
					LastCall_ProduceClientMuteReplicationActivity = MakeTuple(EndpointId, Data);
					return ReturnResult_ProduceClientMuteReplicationActivity;
				}
				
			case EConcertSyncReplicationActivityType::None: [[fallthrough]];
			case EConcertSyncReplicationActivityType::Count: [[fallthrough]];
			default: ensure(false); return {};
			}
		}

		virtual bool GetLastReplicationActivityByClient(const FConcertSessionClientInfo& InClientInfo, EConcertSyncReplicationActivityType ActivityType, FConcertSyncReplicationActivity& OutActivity) const override
		{
			LastCall_GetLastReplicationActivityByClient = MakeTuple(InClientInfo);
			if (const FConcertSyncReplicationActivity* Result = ReturnResult_GetLastReplicationActivityByClient.Find(ActivityType))
			{
				OutActivity = *Result;
				return true;
			}
			return false;
		}

		virtual bool GetReplicationEventById(const int64 ActivityId, FConcertSyncReplicationEvent& OutEvent) const override
		{
			LastCall_GetLeaveReplicationActivityById = MakeTuple(ActivityId);
			if (ReturnResult_GetReplicationEventById)
			{
				OutEvent = *ReturnResult_GetReplicationEventById;
			}
			return ReturnResult_GetReplicationEventById.IsSet();
		}
		
		virtual void EnumerateReplicationActivities(TFunctionRef<EBreakBehavior(const FConcertSyncReplicationActivity& Activity)> Callback) const override
		{
			if (!ReturnResult_EnumerateActivities)
			{
				return;
			}
			
			for (const FConcertSyncReplicationActivity& Activity : *ReturnResult_EnumerateActivities)
			{
				if (Callback(Activity) == EBreakBehavior::Break)
				{
					break;
				}
			}
		}
		//~ End IReplicationWorkspace Interface
	};
}
