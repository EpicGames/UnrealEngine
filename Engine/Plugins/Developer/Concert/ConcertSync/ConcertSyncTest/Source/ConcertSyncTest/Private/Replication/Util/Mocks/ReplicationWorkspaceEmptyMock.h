// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IReplicationWorkspace.h"
#include "Templates/Function.h"

namespace UE::ConcertSyncTests::Replication
{
	class FReplicationWorkspaceEmptyMock : public ConcertSyncServer::Replication::IReplicationWorkspace
	{
	public:
		
		//~ Begin IReplicationWorkspace Interface
		virtual TOptional<int64> ProduceReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationEvent& EventData) override { return {}; }
		virtual bool GetLastReplicationActivityByClient(const FConcertSessionClientInfo& InClientInfo, EConcertSyncReplicationActivityType ActivityType, FConcertSyncReplicationActivity& OutActivity) const override { return false; }
		virtual bool GetReplicationEventById(const int64 ActivityId, FConcertSyncReplicationEvent& OutEvent) const override { return false; }
		virtual void EnumerateReplicationActivities(TFunctionRef<EBreakBehavior(const FConcertSyncReplicationActivity& Activity)> Callback) const override {}
		//~ End IReplicationWorkspace Interface
	};
}
