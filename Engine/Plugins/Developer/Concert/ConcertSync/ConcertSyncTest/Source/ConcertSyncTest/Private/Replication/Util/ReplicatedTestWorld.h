// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertLogGlobal.h"
#include "Replication/Misc/ActorLabelRemappingCore.h"

#include "Algo/AnyOf.h"
#include "Algo/Transform.h"
#include "PreviewScene.h"
#include "Misc/AutomationTest.h"

namespace UE::ConcertSyncTests::Replication
{
	/** Util that has a world and replication map of the objects in it. */
	struct FReplicatedTestWorld
	{
		FPreviewScene PreviewScene;
		FConcertObjectReplicationMap ReplicationMap;

		FReplicatedTestWorld()
			: PreviewScene(FPreviewScene::ConstructionValues()
				.SetTransactional(false)
				.SetCreateDefaultLighting(false)
				.ShouldSimulatePhysics(false)
				.SetCreatePhysicsScene(false)
			)
		{}

		template<typename TActor> 
		TActor* SpawnActor(FName Name, FString Label)
		{
			FActorSpawnParameters Params;
			Params.Name = Name;
			Params.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Required_ErrorAndReturnNull;
			TActor* Actor = PreviewScene.GetWorld()->SpawnActor<TActor>(Params);
			Actor->SetActorLabel(Label);
			return Actor;
		}

		void AddReplicatedProperty(const UObject* Object, FName RootPropertyName)
		{
			AddReplicatedProperties(Object, { FConcertPropertyChain::CreateFromPath(*Object->GetClass(), { RootPropertyName }) });
		}
		void AddReplicatedProperties(const UObject* Object, const TArray<TOptional<FConcertPropertyChain>>& Properties)
		{
			const bool bHasInvalid = Algo::AnyOf(Properties, [](const TOptional<FConcertPropertyChain>& Property){ return !Property.IsSet(); });
			if (bHasInvalid)
			{
				UE_LOGF(LogConcert, Error, "Did not resolve property");
				return;
			}

			FConcertReplicatedObjectInfo& ObjectInfo = ReplicationMap.ReplicatedObjects.FindOrAdd(
				Object, { .ClassPath = Object->GetClass() }
				);
			Algo::Transform(Properties, ObjectInfo.PropertySelection.ReplicatedProperties,
				[](const TOptional<FConcertPropertyChain>& Property){ return *Property; }
				);
		}
	};

	static void TestReplicatesOnlyProperty(const FConcertObjectReplicationMap& Map, const UObject* Object, FName PropertyName, FAutomationTestBase& Test)
	{
		const FConcertReplicatedObjectInfo* ActorData = Map.ReplicatedObjects.Find(Object);
		if (ActorData)
		{
			Test.TestEqual(FString::Printf(TEXT("Class (%s)"), *Object->GetPathName()),
				ActorData->ClassPath, FSoftClassPath(Object->GetClass())
				);
			const TSet<FConcertPropertyChain>& Properties = ActorData->PropertySelection.ReplicatedProperties;
			if (Properties.Num() != 1)
			{
				Test.AddError(TEXT("Expected exactly 1 property"));
			}
			else
			{
				const FConcertPropertyChain& Property = *Properties.CreateConstIterator();
				Test.TestEqual(TEXT("Root name"), Property.GetRootProperty(), PropertyName);
				Test.TestEqual(TEXT("Path length"), Property.GetPathToProperty().Num(), 1);
			}
		}
		else
		{
			Test.AddError(FString::Printf(TEXT("Property %s is not replicated for %s"), *PropertyName.ToString(), *Object->GetPathName()));
		}
	}
}