// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SendReceiveTestBase.h"

struct FConcertReplication_BatchReplicationEvent;
class IConcertClientReplicationManager;
class UTestReflectionObject;

namespace UE::ConcertSyncServer::Replication
{
	class IConcertServerReplicationManager;
}

namespace UE::ConcertSyncTests::Replication
{
	class FConcertClientReplicationBridgeMock;
	
	/**
	 * Simple test exposing SenderArgs and ReceiverArgs and implements CreateSenderArgs and CreateReceiverArgs to return them, respectively.
	 */
	class FSendReceiveGenericTestBase : public FSendReceiveTestBase
	{
	public:

		FSendReceiveGenericTestBase(const FString& InName, const bool bInComplexTask)
			: FSendReceiveTestBase(InName, bInComplexTask)
		{}

	protected:
		
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderArgs;
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs ReceiverArgs;

		//~ Begin FSendReceiveTestBase Interface
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateSenderArgs() override { return SenderArgs; }
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateReceiverArgs() override { return ReceiverArgs; }
		//~ End FSendReceiveTestBase Interface
	};
}
