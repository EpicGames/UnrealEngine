// Copyright Epic Games, Inc. All Rights Reserved.

#include "SendReceiveObjectTestBase.h"

#include "ConcertClientReplicationBridgeMock.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Replication/TestReflectionObject.h"
#include "Util/ClientServerCommunicationTest.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::ConcertSyncTests::Replication
{
	ConcertSyncClient::Replication::FJoinReplicatedSessionArgs FSendReceiveObjectTestBase::CreateSenderArgs()
	{
		return CreateHandshakeArgsFrom(*TestObject, SenderStreamId);
	}

	ConcertSyncClient::Replication::FJoinReplicatedSessionArgs FSendReceiveObjectTestBase::CreateReceiverArgs()
	{
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs ReceiverJoinArgs;
		// TODO DP: When we add client attributes, this test must be updated with attributes that allow receiving all data
		return ReceiverJoinArgs;
	}

	void FSendReceiveObjectTestBase::SetUpClientAndServer(ESendReceiveTestFlags Flags)
	{
		// Fake replicated object must be created before call to Super
		TestObject = NewObject<UTestReflectionObject>(GetTransientPackage());

		FSendReceiveTestBase::SetUpClientAndServer(Flags);
		
		// Bridge is responsible for telling client-side replication system about existing objects
		BridgeMock_Sender->InjectAvailableObject(*TestObject); 
		BridgeMock_Receiver->InjectAvailableObject(*TestObject);
	}
	
	void FSendReceiveObjectTestBase::SimulateSendObjectToReceiver(
		TFunctionRef<FReceiveReplicationEventSignature> OnServerReceive,
		TFunctionRef<FReceiveReplicationEventSignature> OnReceiverClientReceive,
		EPropertyTestFlags PropertyFlags
		)
	{
		auto TestReplicationData_Server = [this, OnServerReceive](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
		{
			const TSet<FGuid> SenderStreamIds = GetSenderStreamIds();
			TestEqual(TEXT("Server received right number of streams"), Event.Streams.Num(), SenderStreamIds.Num());
			for (int32 i = 0; i < Event.Streams.Num(); ++i)
			{
				TestEqual(TEXT("Server received 1 object"), Event.Streams[i].ReplicatedObjects.Num() , 1);
				TestTrue(TEXT("Server received from correct stream"), SenderStreamIds.Contains(Event.Streams[i].StreamId));
				const FSoftObjectPath ObjectPath = Event.Streams[i].ReplicatedObjects.IsEmpty() ? FSoftObjectPath{} : Event.Streams[i].ReplicatedObjects[0].ReplicatedObject;
				TestEqual(TEXT("Server's received object has correct path"), ObjectPath, FSoftObjectPath(TestObject));
			}
			OnServerReceive(Context, Event);
		};
		auto TestReplicationData_Client_Receiver = [this, OnReceiverClientReceive](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
		{
			const TSet<FGuid> SenderStreamIds = GetSenderStreamIds();
			TestEqual(TEXT("Client 2 received right number of streams"), Event.Streams.Num(), SenderStreamIds.Num());
			for (int32 i = 0; i < Event.Streams.Num(); ++i)
			{
				TestEqual(TEXT("Client 2 received 1 object"), Event.Streams[i].ReplicatedObjects.Num() , 1);
				TestTrue(TEXT("Client 2 received from correct stream"), SenderStreamIds.Contains(Event.Streams[i].StreamId));
				const FSoftObjectPath ObjectPath = Event.Streams[i].ReplicatedObjects.IsEmpty() ? FSoftObjectPath{} : Event.Streams[i].ReplicatedObjects[0].ReplicatedObject;
				TestEqual(TEXT("Client 2's received object has correct path"), ObjectPath, FSoftObjectPath(TestObject));
			}
			OnReceiverClientReceive(Context, Event);
		};
		const FDelegateHandle ServerHandle = ServerSession->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(TestReplicationData_Server);
		const FDelegateHandle ClientHandle = Client_Receiver->ClientSessionMock->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(TestReplicationData_Client_Receiver);

		
		// TestObject is the same UObject on both clients.
		// Hence we must override test values with SetTestValues and SetDifferentValues.
		// 1 Sender > Server
		SetTestValues(*TestObject, PropertyFlags);
		TickClient(Client_Sender);
		
		// 2 Forward from server to receiver
		TickServer();
		
		// 3 Receive from server
		SetDifferentValues(*TestObject, PropertyFlags);
		TickClient(Client_Receiver);

		ServerSession->UnregisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(ServerHandle);
		Client_Receiver->ClientSessionMock->UnregisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(ClientHandle);
		
		// No call to Super because we're completely overriding the behavior.
	}

	void FSendReceiveObjectTestBase::SetTestValues(UTestReflectionObject& Object, EPropertyTestFlags PropertyFlags)
	{
		const bool bSendCDOValues = EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::SendCDOValues);
		if (EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::Float))
		{
			Object.Float = bSendCDOValues ? GetMutableDefault<UTestReflectionObject>()->Float : SentFloat;
		}
		if (EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::Vector))
		{
			Object.Vector = bSendCDOValues ? GetMutableDefault<UTestReflectionObject>()->Vector : SentVector;
		}
	}
	
	void FSendReceiveObjectTestBase::SetDifferentValues(UTestReflectionObject& Object, EPropertyTestFlags PropertyFlags)
	{
		if (EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::Float))
		{
			Object.Float = DifferentFloat;
		}
		if (EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::Vector))
		{
			Object.Vector = DifferentVector;
		}
	}
	
	void FSendReceiveObjectTestBase::TestEqualTestValues(UTestReflectionObject& Object, EPropertyTestFlags PropertyFlags)
	{
		const bool bSendCDOValues = EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::SendCDOValues);
		if (EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::Float))
		{
			const float ExpectedValue = bSendCDOValues ? GetMutableDefault<UTestReflectionObject>()->Float : SentFloat;
			TestEqual(TEXT("Float"), Object.Float, ExpectedValue);
		}
		if (EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::Vector))
		{
			const FVector ExpectedValue = bSendCDOValues ? GetMutableDefault<UTestReflectionObject>()->Vector : SentVector;
			TestEqual(TEXT("Vector"), Object.Vector, ExpectedValue);
		}
	}

	void FSendReceiveObjectTestBase::TestEqualDifferentValues(UTestReflectionObject& Object, EPropertyTestFlags PropertyFlags)
	{
		if (EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::Float))
		{
			TestEqual(TEXT("Float"), Object.Float, DifferentFloat);
		}
		if (EnumHasAnyFlags(PropertyFlags, EPropertyTestFlags::Vector))
		{
			TestEqual(TEXT("Vector"), Object.Vector, DifferentVector);
		}
	}
}
