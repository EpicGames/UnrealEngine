// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SendReceiveTestBase.h"

struct FConcertReplication_BatchReplicationEvent;
class IConcertClientReplicationManager;
class UTestReflectionObject;

namespace UE::ConcertSyncServer::Replication
{
	class IConcertServerReplicationManager;
}

namespace UE::ConcertSyncTests::Replication
{
	class FConcertClientReplicationBridgeMock;

	/** Defines the properties to be tested in FSendReceiveObjectTestBase: some tests may want more control of which properties are sent. */
	enum class EPropertyTestFlags
	{
		None,
		
		Float = 1 << 0,
		Vector = 1 << 1,

		/** The CDO values should be sent. */
		SendCDOValues = 1 << 2,
		
		All = Float | Vector
	};
	ENUM_CLASS_FLAGS(EPropertyTestFlags);
	
	/**
	 * Creates a server and connects a sender and receiver client.
	 * The clients will be "sending" an UTestReflectionObject among each other.
	 */
	class FSendReceiveObjectTestBase : public FSendReceiveTestBase
	{
	public:

		FSendReceiveObjectTestBase(const FString& InName, const bool bInComplexTask)
			: FSendReceiveTestBase(InName, bInComplexTask)
		{}
		
		/**
		 * The object the test will be run on.
		 * 
		 * Note: this pointer is knowingly left dangling after the test completes...
		 * there really is not need to override CleanUpTest because we assign TestObject first thing in SetUpClientAndServer.
		 */
		UTestReflectionObject* TestObject = nullptr;
		const FGuid SenderStreamId = FGuid::NewGuid();

		//~ Begin FSendReceiveTestBase Interface
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateSenderArgs() override;
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateReceiverArgs() override;
		virtual void SetUpClientAndServer(ESendReceiveTestFlags Flags = ESendReceiveTestFlags::None) override;
		//~ End FSendReceiveTestBase Interface
		
		/**
		 * Sets test values on TestObject and sends it to the receiver.
		 * If the data arrived, bHasServerReceivedData and bHasClientReceivedData are true.
		 */
		void SimulateSendObjectToReceiver(
			TFunctionRef<FReceiveReplicationEventSignature> OnServerReceive = [](auto&, auto&){},
			TFunctionRef<FReceiveReplicationEventSignature> OnReceiverClientReceive = [](auto&, auto&){},
			EPropertyTestFlags PropertyFlags = EPropertyTestFlags::All
			);

		/** Sets the specified properties to its test values */
		void SetTestValues(UTestReflectionObject& Object, EPropertyTestFlags PropertyFlags = EPropertyTestFlags::All);
		/** Sets the specified properties to values different from the test values */
		void SetDifferentValues(UTestReflectionObject& Object, EPropertyTestFlags PropertyFlags = EPropertyTestFlags::All);

		/** Tests that the specified properties are equal to their test values */
		void TestEqualTestValues(UTestReflectionObject& Object, EPropertyTestFlags PropertyFlags = EPropertyTestFlags::All);
		/** Tests that the specified properties are equal to the values different from the test values (i.e. the values SetDifferentValues sets). */
		void TestEqualDifferentValues(UTestReflectionObject& Object, EPropertyTestFlags PropertyFlags = EPropertyTestFlags::All);
		
	protected:
		
		const float SentFloat = 42.f;
		const FVector SentVector = { 21.f, 84.f, -1.f };
		
		const float DifferentFloat = -420.f;
		const FVector DifferentVector = { -210.f, -840.f, 10.f };
		
		virtual TSet<FGuid> GetSenderStreamIds() const { return { SenderStreamId }; } 
	};
}
