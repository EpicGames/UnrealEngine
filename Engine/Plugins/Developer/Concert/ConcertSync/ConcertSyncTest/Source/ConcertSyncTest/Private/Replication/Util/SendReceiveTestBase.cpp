// Copyright Epic Games, Inc. All Rights Reserved.

#include "SendReceiveTestBase.h"

#include "ConcertClientReplicationBridgeMock.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Replication/Messages/Handshake.h"
#include "Replication/ReplicationTestInterface.h"
#include "Util/ClientServerCommunicationTest.h"

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Replication/PropertyChainUtils.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::ConcertSyncTests::Replication
{
	FSendReceiveTestBase::FSendReceiveTestBase(const FString& InName, const bool bInComplexTask)
			: FConcertClientServerCommunicationTest(InName, bInComplexTask)
	{}

	ConcertSyncClient::Replication::FJoinReplicatedSessionArgs FSendReceiveTestBase::CreateHandshakeArgsFrom(
		const UObject& Object,
		const FGuid& SenderStreamId,
		EConcertObjectReplicationMode ReplicationMode,
		uint8 ReplicationRate
		)
	{
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderJoinArgs;
		
		FConcertReplicatedObjectInfo ReplicatedObjectInfo { Object.GetClass() };
		ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(*Object.GetClass(), [&ReplicatedObjectInfo](FConcertPropertyChain&& Chain)
		{
			ReplicatedObjectInfo.PropertySelection.ReplicatedProperties.Emplace(MoveTemp(Chain));
			return EBreakBehavior::Continue;
		});

		FConcertReplicationStream SendingStream;
		SendingStream.BaseDescription.Identifier = SenderStreamId;
		SendingStream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(&Object, ReplicatedObjectInfo);
		SendingStream.BaseDescription.FrequencySettings.Defaults = { ReplicationMode, ReplicationRate };
		SenderJoinArgs.Streams.Add(SendingStream);
		return SenderJoinArgs;
	}

	void FSendReceiveTestBase::SetUpClientAndServer(ESendReceiveTestFlags Flags)
	{
		// Server
		InitServer();
		ServerSession = GetServerSessionMock();
		ServerReplicationManager = ConcertSyncServer::TestInterface::CreateServerReplicationManager(ServerSession.ToSharedRef(), WorkspaceMock);
		// Client Receiver
		Client_Receiver = &ConnectClient();
		if (EnumHasAnyFlags(Flags, ESendReceiveTestFlags::UseRealReplicationBridge))
		{
			BridgeUsed_Receiver = ConcertSyncClient::TestInterface::CreateClientReplicationBridge();
		}
		else
		{
			BridgeMock_Receiver =  MakeShared<FConcertClientReplicationBridgeMock>();
			BridgeUsed_Receiver = BridgeMock_Receiver;
		}
		ClientReplicationManager_Receiver = ConcertSyncClient::TestInterface::CreateClientReplicationManager(Client_Receiver->ClientSessionMock, *BridgeUsed_Receiver.Get());
		
		// Client Sender
		Client_Sender = &ConnectClient();
		if (EnumHasAnyFlags(Flags, ESendReceiveTestFlags::UseRealReplicationBridge))
		{
			BridgeUsed_Sender = ConcertSyncClient::TestInterface::CreateClientReplicationBridge();
		}
		else
		{
			BridgeMock_Sender =  MakeShared<FConcertClientReplicationBridgeMock>();
			BridgeUsed_Sender = BridgeMock_Sender;
		}
		ClientReplicationManager_Sender = ConcertSyncClient::TestInterface::CreateClientReplicationManager(Client_Sender->ClientSessionMock, *BridgeUsed_Sender.Get());
		
		// 1.1 Sender offers to send all UTestReflectionObject properties
		{
			ClientReplicationManager_Sender->JoinReplicationSession(CreateSenderArgs())
				.Next([this](const ConcertSyncClient::Replication::FJoinReplicatedSessionResult& Result){ TestTrue(TEXT("Sender joined"), Result.ErrorCode == EJoinReplicationErrorCode::Success); });
		}
		// 1.2 Receiver accepts all properties
		{
			ClientReplicationManager_Receiver->JoinReplicationSession(CreateReceiverArgs())
				.Next([this](const ConcertSyncClient::Replication::FJoinReplicatedSessionResult& Result){ TestTrue(TEXT("Receiver joined"), Result.ErrorCode == EJoinReplicationErrorCode::Success); });
		}
		
		// 1.3 Test basic assumptions: only sender sends, only receiver receives.
		auto TestReplicationData_Server = [&](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
		{
			TestEqual(TEXT("Only sender is supposed to send data!"), Context.SourceEndpointId, Client_Sender->ClientSessionMock->GetSessionClientEndpointId());
		};
		auto TestReplicationData_Client_Sender = [&](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
        {
        	// There was a bug where client would receive its own data...
			AddError(TEXT("Sender was not supposed to receive data!"));
        };
		
		ServerSession->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(TestReplicationData_Server);
		Client_Sender->ClientSessionMock->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(TestReplicationData_Client_Sender);
	}
	
	void FSendReceiveTestBase::SimulateSenderToReceiver(
		TFunctionRef<FReceiveReplicationEventSignature> OnServerReceive,
		TFunctionRef<FReceiveReplicationEventSignature> OnReceiverClientReceive
		)
	{
		auto TestReplicationData_Server = [this, OnServerReceive](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
		{
			OnServerReceive(Context, Event);
		};
		auto TestReplicationData_Client_Receiver = [this, OnReceiverClientReceive](const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event)
		{
			OnReceiverClientReceive(Context, Event);
		};
		const FDelegateHandle ServerHandle = ServerSession->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(TestReplicationData_Server);
		const FDelegateHandle ClientHandle = Client_Receiver->ClientSessionMock->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(TestReplicationData_Client_Receiver);

		// 1. Sender > Server
		TickClient(Client_Sender);
		// 2. Forward from server to receiver
		TickServer();
		// 3. Receive from server
		TickClient(Client_Receiver);
		
		ServerSession->UnregisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(ServerHandle);
		Client_Receiver->ClientSessionMock->UnregisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(ClientHandle);
	}
	
	void FSendReceiveTestBase::TickClient(FClientInfo* Client)
	{
		Client->ClientSessionMock->OnTick().Broadcast(*Client->ClientSessionMock, FakeDeltaTime);
	}

	void FSendReceiveTestBase::TickServer()
	{
		ServerSession->OnTick().Broadcast(*ServerSession, FakeDeltaTime);
	}

	void FSendReceiveTestBase::CleanUpTest(FAutomationTestBase* AutomationTestBase)
	{
		if (AutomationTestBase == this)
		{
			ClientReplicationManager_Sender.Reset();
			BridgeMock_Sender.Reset();
			BridgeUsed_Sender.Reset();
			ClientReplicationManager_Receiver.Reset();
			BridgeMock_Receiver.Reset();
			BridgeUsed_Sender.Reset();
			ServerReplicationManager.Reset();
			ServerSession.Reset();
		}

		FConcertClientServerCommunicationTest::CleanUpTest(AutomationTestBase);
	}
}
