// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mocks/ReplicationWorkspaceEmptyMock.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Messages/ObjectReplication.h"
#include "Util/ClientServerCommunicationTest.h"

class IConcertClientReplicationBridge;
class IConcertClientReplicationManager;
class UTestReflectionObject;

namespace UE::ConcertSyncClient::Replication
{
	struct FJoinReplicatedSessionArgs;
}
namespace UE::ConcertSyncServer::Replication
{
	class IConcertServerReplicationManager;
}

namespace UE::ConcertSyncTests::Replication
{
	class FConcertClientReplicationBridgeMock;

	enum class ESendReceiveTestFlags : uint8
	{
		None = 0,
		/**
		 * Instead of mocking the bridge use the real implementation. The object will only be discovered if it exists when the stream is registered.
		 * See IConcertClientReplicationBridge::PushTrackedObjects.
		 */
		UseRealReplicationBridge = 1 << 1
	};
	ENUM_CLASS_FLAGS(ESendReceiveTestFlags);
	
	/** Creates a server, connects a sender and receiver client, and completes a handshake for them. */
	class FSendReceiveTestBase : public FConcertClientServerCommunicationTest
	{
	public:
	
		FSendReceiveTestBase(const FString& InName, const bool bInComplexTask);

		/**
		 * Creates joining arguments containing a stream that will replicate a single object.
		 * @param Object The object to replicate
		 * @param SenderStreamId The ID of the new stream
		 * @param ReplicationMode The replication mode for the object. Set to Realtime by default so tests need not handle latency (due to FConcertReplicatedObjectInfo::ReplicationRate).
		 * @param ReplicationRate The replication rate for the object. Only used if ReplicationMode == EConcertObjectReplicationMode::SpecifiedRate
		 */
		static ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateHandshakeArgsFrom(
			const UObject& Object,
			const FGuid& SenderStreamId = FGuid::NewGuid(),
			EConcertObjectReplicationMode ReplicationMode = EConcertObjectReplicationMode::Realtime,
			uint8 ReplicationRate = 30
			);
		
		using FReceiveReplicationEventSignature = void(const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event);
		
		float FakeDeltaTime = 1.f / 60.f;
		
		TSharedPtr<FConcertServerSessionMock> ServerSession;
		FReplicationWorkspaceEmptyMock WorkspaceMock;
		TSharedPtr<ConcertSyncServer::Replication::IConcertServerReplicationManager> ServerReplicationManager;
		
		FClientInfo* Client_Receiver = nullptr;
		/** This is null if ESendReceiveTestFlags::UseRealReplicationBridge was specified. */
		TSharedPtr<FConcertClientReplicationBridgeMock> BridgeMock_Receiver;
		/** This is always valid. It is the bridge being used by the receiver. */
		TSharedPtr<IConcertClientReplicationBridge> BridgeUsed_Receiver;
		TSharedPtr<IConcertClientReplicationManager> ClientReplicationManager_Receiver;

		FClientInfo* Client_Sender  = nullptr;
		/** This is null if ESendReceiveTestFlags::UseRealReplicationBridge was specified. */
		TSharedPtr<FConcertClientReplicationBridgeMock> BridgeMock_Sender;
		/** This is always valid. It is the bridge being used by the receiver. */
		TSharedPtr<IConcertClientReplicationBridge> BridgeUsed_Sender;
		TSharedPtr<IConcertClientReplicationManager> ClientReplicationManager_Sender;

		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateSenderArgs() = 0;
		virtual ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateReceiverArgs() = 0;

		virtual void SetUpClientAndServer(ESendReceiveTestFlags Flags = ESendReceiveTestFlags::None);
		virtual void SimulateSenderToReceiver(
			TFunctionRef<FReceiveReplicationEventSignature> OnServerReceive = [](auto&, auto&){},
			TFunctionRef<FReceiveReplicationEventSignature> OnReceiverClientReceive = [](auto&, auto&){}
			);
		
		void TickClient(FClientInfo* Client);
		void TickServer();

		//~ Begin FConcertClientServerCommunicationTest Interface
		virtual void CleanUpTest(FAutomationTestBase* AutomationTestBase) override;
		//~ End FConcertClientServerCommunicationTest Interface
	};
}
