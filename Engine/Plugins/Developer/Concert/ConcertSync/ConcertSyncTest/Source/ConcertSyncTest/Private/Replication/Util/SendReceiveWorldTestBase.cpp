// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Util/SendReceiveWorldTestBase.h"

namespace UE::ConcertSyncTests::Replication
{
	void FSendReceiveWorldTestBase::SetupWorld()
	{
		WorldManager = MakeUnique<FPreviewScene>(
			FPreviewScene::ConstructionValues()
			.SetEditor(true)
			);
	}

	UWorld* FSendReceiveWorldTestBase::GetWorld() const
	{
		return ensureMsgf(WorldManager.IsValid(), TEXT("Your test did not call InitWorld!"))
			? WorldManager->GetWorld()
			: nullptr;
	}

	void FSendReceiveWorldTestBase::SimulateSenderToReceiverForWorld(TFunctionRef<void(UWorld& World)> SetTestValues, TFunctionRef<void(UWorld& World)> SetGarbageValues, TOptional<TFunctionRef<void(UWorld& World)>> TestValues)
	{
		UWorld* World = GetWorld();
		if (!ensureMsgf(World, TEXT("Your test did not call InitWorld!")))
		{
			return;
		}
		
		// TestObject is the same UObject on both clients.
		// Hence we must override test values with SetTestValues and SetDifferentValues.
		// 1 Sender > Server
		SetTestValues(*World);
		TickClient(Client_Sender);
		
		// 2 Forward from server to receiver
		TickServer();
		
		// 3 Receive from server
		SetGarbageValues(*World);
		TickClient(Client_Receiver);
		
		if (TestValues)
		{
			TestValues.GetValue()(*World);
		}
		// No call to Super because we're completely overriding the behavior.
	}

	void FSendReceiveWorldTestBase::CleanUpTest(FAutomationTestBase* AutomationTestBase)
	{
		WorldManager.Reset();
		Super::CleanUpTest(AutomationTestBase);
	}
}