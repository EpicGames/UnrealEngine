// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreviewScene.h"
#include "SendReceiveGenericStreamTestBase.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

class UWorld;

namespace UE::ConcertSyncTests::Replication
{
	class FConcertClientReplicationBridgeMock;
	
	/** For tests that want to test replication in a UWorld. */
	class FSendReceiveWorldTestBase : public FSendReceiveGenericTestBase
	{
		using Super = FSendReceiveGenericTestBase;
	public:

		FSendReceiveWorldTestBase(const FString& InName, const bool bInComplexTask)
			: Super(InName, bInComplexTask)
		{}

		/** Called by test when it is ready to initialize the world. */
		virtual void SetupWorld();

		/** @return The test's world. */
		UWorld* GetWorld() const;

		/**
		 * Simulates sending values from one world to the other.
		 * 
		 * This actually operates on the same UWorld instance so that FSoftObjectPaths continue working.
		 * This is how to use this function:
		 * 1. Set up all your replicated objects
		 * 2. SetTestValues sets values on whatever world objects you want
		 * 3. This function reads said data and replicates it to the server
		 * 4. The server forwards it to the receiving client
		 * 5. SetGarbageValues sets values that are different from the test values
		 * 6. This function applies the replicated data thus overriding the garbage values (that is assuming replication worked as expected!)
		 * 7. This function executes TestValues if set so you can test whether the data was indeed replicated correctly.
		 * 
		 * @param SetTestValues Modifies values on replicated properties before it it sent from the sender client
		 * @param SetGarbageValues Modifies values on replicated properties before the replicated data is applied on the receiving client
		 * @param TestValues Optionally tests that the replicated data was applied correctly.
		 * 
		 */
		void SimulateSenderToReceiverForWorld(
			TFunctionRef<void(UWorld& World)> SetTestValues,
			TFunctionRef<void(UWorld& World)> SetGarbageValues,
			TOptional<TFunctionRef<void(UWorld& World)>> TestValues = {}
			);

	private:

		/** Set when the test is started. Reset by CleanUpTest. */
		TUniquePtr<FPreviewScene> WorldManager;
		
		//~ Begin FConcertClientServerCommunicationTest Interface
		virtual void CleanUpTest(FAutomationTestBase* AutomationTestBase) override;
		//~ End FConcertClientServerCommunicationTest Interface
	};
}
