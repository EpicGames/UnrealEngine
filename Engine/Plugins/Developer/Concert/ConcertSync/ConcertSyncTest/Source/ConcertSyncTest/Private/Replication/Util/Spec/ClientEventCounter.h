// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ReplicationClient.h"

namespace UE::ConcertSyncTests::Replication
{
	/** Helps track the number of authority and stream change events in IConcertClientReplicationManager.  */
	struct FClientEventCounter
	{
		int32 CallCount_PreAuthorityChanged = 0;
		int32 CallCount_PostAuthorityChanged = 0;
		int32 CallCount_PreStreamsChanged = 0;
		int32 CallCount_PostStreamsChanged = 0;
		
		void ResetEventCount()
		{
			CallCount_PreAuthorityChanged = 0;
			CallCount_PostAuthorityChanged = 0;
			CallCount_PreStreamsChanged = 0;
			CallCount_PostStreamsChanged = 0;
		}
		
		void TestCount(FAutomationTestBase& Test, int32 ExpectedStreamCount, int32 ExpectedAuthorityCount) const
		{
			Test.TestEqual(TEXT("PreAuthorityChanged"), CallCount_PreAuthorityChanged, ExpectedAuthorityCount);
			Test.TestEqual(TEXT("PostAuthorityChanged"), CallCount_PostAuthorityChanged, ExpectedAuthorityCount);
			Test.TestEqual(TEXT("PreStreamsChanged"), CallCount_PreStreamsChanged, ExpectedStreamCount);
			Test.TestEqual(TEXT("PostStreamsChanged"), CallCount_PostStreamsChanged, ExpectedStreamCount);
		}

		void Subscribe(const FReplicationClient& Client)
		{
			Client.GetClientReplicationManager().OnPreAuthorityChanged().AddLambda([this]{ ++CallCount_PreAuthorityChanged; });
			Client.GetClientReplicationManager().OnPostAuthorityChanged().AddLambda([this]{ ++CallCount_PostAuthorityChanged; });
			Client.GetClientReplicationManager().OnPreStreamsChanged().AddLambda([this]{ ++CallCount_PreStreamsChanged; });
			Client.GetClientReplicationManager().OnPostStreamsChanged().AddLambda([this]{ ++CallCount_PostStreamsChanged; });
			ResetEventCount();
		}
	};
}
