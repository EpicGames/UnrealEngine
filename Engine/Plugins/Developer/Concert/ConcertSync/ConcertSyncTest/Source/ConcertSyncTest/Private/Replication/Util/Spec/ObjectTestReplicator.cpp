// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTestReplicator.h"

#include "ReplicationClient.h"
#include "ReplicationServer.h"
#include "Replication/PropertyChainUtils.h"
#include "Replication/Messages/ObjectReplication.h"

namespace UE::ConcertSyncTests::Replication
{
	namespace Private
	{
		FString TestName(const TCHAR* BaseName, const TCHAR* Context)
		{
			return Context
				? FString::Printf(TEXT("%s: %s"), Context, BaseName)
				: BaseName;
		}
		
		FString TestName(const TCHAR* BaseName, const FObjectReplicationContext& Context)
		{
			return TestName(BaseName, Context.ContextName);
		}
	}

	TSharedRef<FObjectTestReplicator> FObjectTestReplicator::CreateSubobjectReplicator(const FName BaseName) const
	{
		UTestReflectionObject* Subobject = NewObject<UTestReflectionObject>(TestObject, BaseName);
		TestObject->InstancedSubobject = Subobject;
		return MakeShared<FObjectTestReplicator>(Subobject);
	}

	ConcertSyncClient::Replication::FJoinReplicatedSessionArgs FObjectTestReplicator::CreateSenderArgs(
		FGuid SenderStreamId,
		EConcertObjectReplicationMode ReplicationMode,
		uint8 ReplicationRate
		) const
	{
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs SenderJoinArgs;
		SenderJoinArgs.Streams.Add(CreateStream(SenderStreamId, ReplicationMode, ReplicationRate));
		return SenderJoinArgs;
	}

	FConcertReplicationStream FObjectTestReplicator::CreateStream(FGuid SenderStreamId, EConcertObjectReplicationMode ReplicationMode, uint8 ReplicationRate) const
	{
		FConcertReplicationStream SendingStream;
		SendingStream.BaseDescription.Identifier = SenderStreamId;
		SendingStream.BaseDescription.FrequencySettings.Defaults = { ReplicationMode, ReplicationRate };
		AddToStream(SendingStream);
		return SendingStream;
	}

	FConcertReplicationStream FObjectTestReplicator::CreateStreamWithProperties(FGuid SenderStreamId, EPropertyTypeFlags PropertyTypeFlags, EConcertObjectReplicationMode ReplicationMode, uint8 ReplicationRate) const
	{
		FConcertReplicationStream SendingStream;
		SendingStream.BaseDescription.Identifier = SenderStreamId;
		SendingStream.BaseDescription.FrequencySettings.Defaults = { ReplicationMode, ReplicationRate };
		AddToStreamWithProperties(SendingStream, PropertyTypeFlags);
		return SendingStream;
	}

	void FObjectTestReplicator::AddToStream(FConcertReplicationStream& Stream, EConcertObjectReplicationMode ReplicationMode, uint8 ReplicationRate) const
	{
		FConcertReplicatedObjectInfo ReplicatedObjectInfo { TestObject->GetClass() };
		ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(*TestObject->GetClass(), [&ReplicatedObjectInfo](FConcertPropertyChain&& Chain)
		{
			ReplicatedObjectInfo.PropertySelection.ReplicatedProperties.Emplace(MoveTemp(Chain));
			return EBreakBehavior::Continue;
		});
		Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(TestObject, ReplicatedObjectInfo);

		const FConcertObjectReplicationSettings FrequencySettings { ReplicationMode, ReplicationRate };
		if (Stream.BaseDescription.FrequencySettings.Defaults != FrequencySettings)
		{
			Stream.BaseDescription.FrequencySettings.ObjectOverrides.Add(TestObject, FrequencySettings);
		}
	}

	void FObjectTestReplicator::AddToStreamWithProperties(FConcertReplicationStream& Stream, EPropertyTypeFlags PropertyTypeFlags, EConcertObjectReplicationMode ReplicationMode, uint8 ReplicationRate) const
	{
		FConcertReplicatedObjectInfo ReplicatedObjectInfo { TestObject->GetClass() };
		ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(*TestObject->GetClass(), [&ReplicatedObjectInfo, PropertyTypeFlags](FConcertPropertyChain&& Chain)
		{
			const bool bIsFloat = Chain == TArray<FName>{ TEXT("Float") };
			const bool bIsVector = Chain.GetRootProperty() == TEXT("Vector");
			if (!EnumHasAnyFlags(PropertyTypeFlags, EPropertyTypeFlags::Float)
				&& bIsFloat)
			{
				return EBreakBehavior::Continue;
			}
			if (!EnumHasAnyFlags(PropertyTypeFlags, EPropertyTypeFlags::Vector)
				&& bIsVector)
			{
				return EBreakBehavior::Continue;
			}
			if (!EnumHasAnyFlags(PropertyTypeFlags, EPropertyTypeFlags::Others) && !bIsFloat && !bIsVector)
			{
				return EBreakBehavior::Continue;
			}
			
			ReplicatedObjectInfo.PropertySelection.ReplicatedProperties.Emplace(MoveTemp(Chain));
			return EBreakBehavior::Continue;
		});
		Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Add(TestObject, ReplicatedObjectInfo);

		const FConcertObjectReplicationSettings FrequencySettings { ReplicationMode, ReplicationRate };
		if (Stream.BaseDescription.FrequencySettings.Defaults != FrequencySettings)
		{
			Stream.BaseDescription.FrequencySettings.ObjectOverrides.Add(TestObject, FrequencySettings);
		}
	}

	void FObjectTestReplicator::SimulateSendObjectToReceiver(
		FAutomationTestBase& Test,
		FObjectReplicationContext Context,
		TConstArrayView<FGuid> SenderStreams,
		TFunctionRef<FReceiveReplicationEventSignature> OnServerReceive,
		TFunctionRef<FReceiveReplicationEventSignature> OnReceiverClientReceive,
		EPropertyReplicationFlags PropertyFlags
		) const
	{
		using namespace Private;
		const auto TestReplicationData_Server = [this, &Test, &Context, &SenderStreams, &OnServerReceive](const FConcertSessionContext& SessionContext, const FConcertReplication_BatchReplicationEvent& Event)
		{
			Test.TestEqual(TestName(TEXT("Server received right number of streams"), Context), Event.Streams.Num(), SenderStreams.Num());
			for (int32 i = 0; i < Event.Streams.Num(); ++i)
			{
				Test.TestEqual(TestName(TEXT("Server received 1 object"), Context), Event.Streams[i].ReplicatedObjects.Num() , 1);
				Test.TestTrue(TestName(TEXT("Server received from correct stream"), Context), SenderStreams.Contains(Event.Streams[i].StreamId));
				const FSoftObjectPath ObjectPath = Event.Streams[i].ReplicatedObjects.IsEmpty() ? FSoftObjectPath{} : Event.Streams[i].ReplicatedObjects[0].ReplicatedObject;
				Test.TestEqual(TestName(TEXT("Server's received object has correct path"), Context), ObjectPath, FSoftObjectPath(TestObject));
			}
			OnServerReceive(SessionContext, Event);
		};
		const auto TestReplicationData_Client_Receiver = [this, &Test, &Context, &SenderStreams, &OnReceiverClientReceive](const FConcertSessionContext& SessionContext, const FConcertReplication_BatchReplicationEvent& Event)
		{
			Test.TestEqual(TestName(TEXT("Client 2 received right number of streams"), Context), Event.Streams.Num(), SenderStreams.Num());
			for (int32 i = 0; i < Event.Streams.Num(); ++i)
			{
				Test.TestEqual(TestName(TEXT("Client 2 received 1 object"), Context), Event.Streams[i].ReplicatedObjects.Num() , 1);
				Test.TestTrue(TestName(TEXT("Client 2 received from correct stream"), Context), SenderStreams.Contains(Event.Streams[i].StreamId));
				const FSoftObjectPath ObjectPath = Event.Streams[i].ReplicatedObjects.IsEmpty() ? FSoftObjectPath{} : Event.Streams[i].ReplicatedObjects[0].ReplicatedObject;
				Test.TestEqual(TestName(TEXT("Client 2's received object has correct path"), Context), ObjectPath, FSoftObjectPath(TestObject));
			}
			OnReceiverClientReceive(SessionContext, Event);
		};
		const FDelegateHandle ServerHandle = Context.Server.GetServerSessionMock()->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(TestReplicationData_Server);
		const FDelegateHandle ClientHandle = Context.Receiver.GetClientSessionMock()->RegisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(TestReplicationData_Client_Receiver);

		
		// TestObject is the same UObject on both clients.
		// Hence we must override test values with SetTestValues and SetDifferentValues.
		// 1 Sender > Server
		SetTestValues(PropertyFlags);
		Context.Sender.TickClient();
		
		// 2 Forward from server to receiver
		Context.Server.TickServer();
		
		// 3 Receive from server
		SetDifferentValues(PropertyFlags);
		Context.Receiver.TickClient();

		Context.Server.GetServerSessionMock()->UnregisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(ServerHandle);
		Context.Receiver.GetClientSessionMock()->UnregisterCustomEventHandler<FConcertReplication_BatchReplicationEvent>(ClientHandle);
	}

	void FObjectTestReplicator::SetTestValues(EPropertyReplicationFlags PropertyFlags) const
	{
		const bool bSendCDOValues = EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::SendCDOValues);
		if (EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::Float))
		{
			TestObject->Float = bSendCDOValues ? GetMutableDefault<UTestReflectionObject>()->Float : SentFloat;
		}
		if (EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::Vector))
		{
			TestObject->Vector = bSendCDOValues ? GetMutableDefault<UTestReflectionObject>()->Vector : SentVector;
		}
	}

	void FObjectTestReplicator::SetDifferentValues(EPropertyReplicationFlags PropertyFlags) const
	{
		if (EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::Float))
		{
			TestObject->Float = DifferentFloat;
		}
		if (EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::Vector))
		{
			TestObject->Vector = DifferentVector;
		}
	}

	void FObjectTestReplicator::TestValuesWereReplicated(FAutomationTestBase& Test, EPropertyReplicationFlags PropertyFlags, const TCHAR* Context) const
	{
		using namespace Private;
		
		const bool bSendCDOValues = EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::SendCDOValues);
		if (EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::Float))
		{
			const float ExpectedValue = bSendCDOValues ? GetMutableDefault<UTestReflectionObject>()->Float : SentFloat;
			const bool bWasReplicated = Test.TestEqual(TestName(TEXT("Float"), Context), TestObject->Float, ExpectedValue);
			Test.AddErrorIfFalse(bWasReplicated, TestName(TEXT("Failed to replicate \"Float\" property"), Context));
		}
		if (EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::Vector))
		{
			const FVector ExpectedValue = bSendCDOValues ? GetMutableDefault<UTestReflectionObject>()->Vector : SentVector;
			const bool bWasReplicated = Test.TestEqual(TestName(TEXT("Vector"), Context), TestObject->Vector, ExpectedValue);
			Test.AddErrorIfFalse(bWasReplicated, TestName(TEXT("Failed to replicate \"Vector\" property"), Context));
		}
	}

	void FObjectTestReplicator::TestValuesWereNotReplicated(FAutomationTestBase& Test, EPropertyReplicationFlags PropertyFlags, const TCHAR* Context) const
	{
		using namespace Private;
		
		if (EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::Float))
		{
			const bool bWasNotReplicated = Test.TestEqual(TestName(TEXT("Float"), Context), TestObject->Float, DifferentFloat);
			Test.AddErrorIfFalse(bWasNotReplicated, TestName(TEXT("Probably \"Float\" property was replicated even though it was not supposed to be!"), Context));
		}
		if (EnumHasAnyFlags(PropertyFlags, EPropertyReplicationFlags::Vector))
		{
			const bool bWasNotReplicated = Test.TestEqual(TestName(TEXT("Vector"), Context), TestObject->Vector, DifferentVector);
			Test.AddErrorIfFalse(bWasNotReplicated, TestName(TEXT("Probably \"Vector\" property was replicated even though it was not supposed to be!"), Context));
		}
	}
}
