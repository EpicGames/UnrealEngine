// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/TestReflectionObject.h"

#include "Templates/UnrealTemplate.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

struct FConcertReplication_BatchReplicationEvent;
struct FConcertSessionContext;

namespace UE::ConcertSyncTests::Replication
{
	class FReplicationServer;
	class FReplicationClient;

	/** Defines the properties to be tested in FSendReceiveObjectTestBase: some tests may want more control of which properties are sent. */
	enum class EPropertyReplicationFlags : uint8
	{
		None,
		
		Float = 1 << 0,
		Vector = 1 << 1,

		/** The CDO values should be sent. */
		SendCDOValues = 1 << 2,
		
		All = Float | Vector
	};
	ENUM_CLASS_FLAGS(EPropertyReplicationFlags);

	/** Defines the properties you want to add to the stream */
	enum class EPropertyTypeFlags : uint8
	{
		None,
		Float = 1 << 0,
		Vector = 1 << 1,
		Others = 1 << 2,
		All = Float | Vector | Others
	};
	ENUM_CLASS_FLAGS(EPropertyTypeFlags);

	struct FObjectReplicationContext
	{
		FReplicationClient& Sender;
		FReplicationServer& Server;
		FReplicationClient& Receiver;
		/** Helps debug failing tests. */
		const TCHAR* ContextName = nullptr;
	};
	
	/**
	 * This utility is used in the context of 1 FReplicationServer and 2 FReplicationClients and helps testing transmission of properties from UTestReflectionObject.
	 *
	 * It assumes you set up
	 */
	class FObjectTestReplicator : public FNoncopyable
	{
	public:
		
		using FReceiveReplicationEventSignature = void(const FConcertSessionContext& Context, const FConcertReplication_BatchReplicationEvent& Event);

		// Test values
		float SentFloat = 42.f;
		FVector SentVector = { 21.f, 84.f, -1.f };
		float DifferentFloat = -420.f;
		FVector DifferentVector = { -210.f, -840.f, 10.f };

		/** The object that will be transmitted. */
		UTestReflectionObject* TestObject = NewObject<UTestReflectionObject>(GetTransientPackage());

		FObjectTestReplicator() = default;
		explicit FObjectTestReplicator(UTestReflectionObject* TestObject) : TestObject(TestObject) {}
		explicit FObjectTestReplicator(const FName BaseName) : TestObject(NewObject<UTestReflectionObject>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UTestReflectionObject::StaticClass(), BaseName))) {}
		explicit FObjectTestReplicator(UPackage* Package) : TestObject(NewObject<UTestReflectionObject>(Package)) {}

		/** Create a subobject of TestObject, assigns it to TestObject->InstancedSubobject, and returns a replicator for replicating it. */
		TSharedRef<FObjectTestReplicator> CreateSubobjectReplicator(const FName BaseName = NAME_None) const;

		/** Util for creating join args for replicating TestObject. */
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs CreateSenderArgs(
			FGuid SenderStreamId = FGuid::NewGuid(),
			EConcertObjectReplicationMode ReplicationMode = EConcertObjectReplicationMode::Realtime,
			uint8 ReplicationRate = 30
			) const;
		
		/** Util for creating a stream that replicates TestObject. */
		FConcertReplicationStream CreateStream(
			FGuid SenderStreamId = FGuid::NewGuid(),
			EConcertObjectReplicationMode ReplicationMode = EConcertObjectReplicationMode::Realtime,
			uint8 ReplicationRate = 30
		) const;
		/** Util for creating a stream that replicates TestObject and specifies the properties. */
		FConcertReplicationStream CreateStreamWithProperties(
			FGuid SenderStreamId = FGuid::NewGuid(),
			EPropertyTypeFlags PropertyTypeFlags = EPropertyTypeFlags::All,
			EConcertObjectReplicationMode ReplicationMode = EConcertObjectReplicationMode::Realtime,
			uint8 ReplicationRate = 30
		) const;
		
		/** Util adding replication settings for TestObject to a stream */
		void AddToStream(
			FConcertReplicationStream& Stream,
			EConcertObjectReplicationMode ReplicationMode = EConcertObjectReplicationMode::Realtime,
			uint8 ReplicationRate = 30
			) const;
		/** Util adding replication settings for TestObject to a stream and specifies the properties. */
		void AddToStreamWithProperties(
			FConcertReplicationStream& Stream,
			EPropertyTypeFlags PropertyTypeFlags = EPropertyTypeFlags::All,
			EConcertObjectReplicationMode ReplicationMode = EConcertObjectReplicationMode::Realtime,
			uint8 ReplicationRate = 30
			) const;
		
		/** This overload allows you to send properties but from multiple streams. */
		void SimulateSendObjectToReceiver(
			FAutomationTestBase& Test,
			FObjectReplicationContext Context,
			TConstArrayView<FGuid> SenderStreams,
			TFunctionRef<FReceiveReplicationEventSignature> OnServerReceive = [](auto&, auto&){},
			TFunctionRef<FReceiveReplicationEventSignature> OnReceiverClientReceive = [](auto&, auto&){},
			EPropertyReplicationFlags PropertyFlags = EPropertyReplicationFlags::All
			) const;

		/** Sets the specified properties to its test values */
		void SetTestValues(EPropertyReplicationFlags PropertyFlags = EPropertyReplicationFlags::All) const;
		/** Sets the specified properties to values different from the test values */
		void SetDifferentValues(EPropertyReplicationFlags PropertyFlags = EPropertyReplicationFlags::All) const;

		
		/** Tests that the specified properties are equal to their test values */
		void TestValuesWereReplicated(FAutomationTestBase& Test, EPropertyReplicationFlags PropertyFlags = EPropertyReplicationFlags::All, const TCHAR* Context = nullptr) const;
		/** Tests that the specified properties are equal to the values different from the test values (i.e. the values SetDifferentValues sets). */
		void TestValuesWereNotReplicated(FAutomationTestBase& Test, EPropertyReplicationFlags PropertyFlags = EPropertyReplicationFlags::All, const TCHAR* Context = nullptr) const;
		
		void TestValuesWereReplicated(FAutomationTestBase& Test, const TCHAR* Context, EPropertyReplicationFlags PropertyFlags = EPropertyReplicationFlags::All) const { TestValuesWereReplicated(Test, PropertyFlags, Context); }
		void TestValuesWereNotReplicated(FAutomationTestBase& Test, const TCHAR* Context, EPropertyReplicationFlags PropertyFlags = EPropertyReplicationFlags::All) const { TestValuesWereNotReplicated(Test, PropertyFlags, Context); }
	};
}

