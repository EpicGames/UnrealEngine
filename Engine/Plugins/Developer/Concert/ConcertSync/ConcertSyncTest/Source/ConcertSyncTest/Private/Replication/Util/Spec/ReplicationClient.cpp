// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationClient.h"

#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/ReplicationTestInterface.h"
#include "Replication/Util/ConcertClientReplicationBridgeMock.h"

#include "UObject/Object.h"

namespace UE::ConcertSyncTests::Replication
{
	void FReplicationClient::TickClient(float FakeDeltaTime)
	{
		ClientSessionMock->OnTick().Broadcast(*ClientSessionMock, FakeDeltaTime);
	}

	TFuture<ConcertSyncClient::Replication::FJoinReplicatedSessionResult> FReplicationClient::JoinReplication(
		ConcertSyncClient::Replication::FJoinReplicatedSessionArgs Args,
		EReplicationClientFlags TestFlags
		)
	{
		if (EnumHasAnyFlags(TestFlags, EReplicationClientFlags::UseRealReplicationBridge))
		{
			BridgeUsed = ConcertSyncClient::TestInterface::CreateClientReplicationBridge();
		}
		else
		{
			BridgeMock =  MakeShared<FConcertClientReplicationBridgeMock>();
			BridgeUsed = BridgeMock;
		}
		ClientReplicationManager = ConcertSyncClient::TestInterface::CreateClientReplicationManager(ClientSessionMock, *BridgeUsed, SessionFlags);
		
		// 1.1 Sender offers to send all UTestReflectionObject properties
		bool bSuccess = false;
		TFuture<ConcertSyncClient::Replication::FJoinReplicatedSessionResult> Future = ClientReplicationManager->JoinReplicationSession(Args)
			.Next([&bSuccess](ConcertSyncClient::Replication::FJoinReplicatedSessionResult&& Result)
			{
				bSuccess = Result.ErrorCode == EJoinReplicationErrorCode::Success;
				return Result;
			});

		if (!EnumHasAnyFlags(TestFlags, EReplicationClientFlags::SkipJoinTest))
		{
			TestContext.TestTrue(TEXT("Replication joined successfully"), bSuccess);
		}
		
		return Future;
	}

	TFuture<ConcertSyncClient::Replication::FJoinReplicatedSessionResult> FReplicationClient::JoinReplicationAsListener(
		TArray<UObject*> ObjectsToReceive
		)
	{
		return JoinReplication()
			.Next([this, ObjectsToReceive = MoveTemp(ObjectsToReceive)](ConcertSyncClient::Replication::FJoinReplicatedSessionResult&& Result)
			{
				if (Result.ErrorCode != EJoinReplicationErrorCode::Success)
				{
					return Result;
				}
				
				for (UObject* Object : ObjectsToReceive)
				{
					BridgeMock->InjectAvailableObject(*Object);
				}
				return Result;
			});
	}

	void FReplicationClient::LeaveReplication() const
	{
		ClientReplicationManager->LeaveReplicationSession();
	}
}

