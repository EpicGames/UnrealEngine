// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "ConcertSyncSessionFlags.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Util/ClientServerCommunicationTest.h"
#include "Templates/UnrealTemplate.h"

class IConcertClientReplicationManager;
class IConcertClientReplicationBridge;
class UObject;

namespace UE::ConcertSyncTests::Replication
{
	class FConcertClientReplicationBridgeMock;

	enum class EReplicationClientFlags : uint8
	{
		None = 0,
		/**
		 * Instead of mocking the bridge use the real implementation. The object will only be discovered if it exists when the stream is registered.
		 * See IConcertClientReplicationBridge::PushTrackedObjects.
		 */
		UseRealReplicationBridge = 1 << 0,

		/** Don't test whether joining was successful */
		SkipJoinTest = 1 << 1
	};
	ENUM_CLASS_FLAGS(EReplicationClientFlags);
	
	/** Implements reusable behaviour for replication tests, such as the handshake. */
	class FReplicationClient : public FNoncopyable
	{
	public:
		
		FReplicationClient(const FGuid& ClientEndPointId, EConcertSyncSessionFlags SessionFlags, FConcertServerSessionMock& Server, FAutomationTestBase& TestContext, FConcertClientInfo ClientInfo = {})
			: ClientEndPointId(ClientEndPointId)
			, SessionFlags(SessionFlags)
			, TestContext(TestContext)
			, ClientSessionMock(MakeShared<FConcertClientSessionMock>(ClientEndPointId, Server, MoveTemp(ClientInfo)))
		{}
		
		/** Lets the client process any messages that have come in. */
		void TickClient(float FakeDeltaTime = 1.f / 60.f);

		/** Joins the client into replication allowing them to participate in replication */
		TFuture<ConcertSyncClient::Replication::FJoinReplicatedSessionResult> JoinReplication(
			ConcertSyncClient::Replication::FJoinReplicatedSessionArgs Args = {},
			EReplicationClientFlags TestFlags = EReplicationClientFlags::None
			);
		/** This overload joins replication and injects the objects into the FConcertClientReplicationBridgeMock so they can be received. */
		TFuture<ConcertSyncClient::Replication::FJoinReplicatedSessionResult> JoinReplicationAsListener(
			TArray<UObject*> ObjectsToReceive
			);

		/** Leaves the session's replication. */
		void LeaveReplication() const;

		const FGuid& GetEndpointId() const { return ClientEndPointId; }
		const FConcertClientInfo& GetClientInfo() const { return ClientSessionMock->GetLocalClientInfo(); }
		
		TSharedRef<FConcertClientSessionBaseMock> GetClientSessionMock() const { return ClientSessionMock; }
		FConcertClientReplicationBridgeMock& GetBridgeMock() const { checkf(BridgeMock, TEXT("You forgot to call JoinReplication")); return *BridgeMock; }
		IConcertClientReplicationManager& GetClientReplicationManager() const { checkf(ClientReplicationManager, TEXT("You forgot to call JoinReplication")); return *ClientReplicationManager; }

	private:

		/** The client's endpoint. */
		const FGuid ClientEndPointId;
		/** Relevant for certain requests. Passed to ClientReplicationManager upon creation. */
		const EConcertSyncSessionFlags SessionFlags;

		/** Used to test "obvious" cases that should never fail in any test. */
		FAutomationTestBase& TestContext;
		/** The underlying client session */
		const TSharedRef<FConcertClientSessionBaseMock> ClientSessionMock;
		
		/** Manually detects when objects are "available". This is null if ESendReceiveTestFlags::UseRealReplicationBridge was specified. */
		TSharedPtr<FConcertClientReplicationBridgeMock> BridgeMock;
		/** Detects when objects are available. This is always valid. It is the bridge being used by the receiver. */
		TSharedPtr<IConcertClientReplicationBridge> BridgeUsed;
		/** Manages replication client side */
		TSharedPtr<IConcertClientReplicationManager> ClientReplicationManager;
	};
}
