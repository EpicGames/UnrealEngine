// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationServer.h"

#include "ReplicationClient.h"
#include "Replication/ReplicationTestInterface.h"

namespace UE::ConcertSyncTests::Replication
{
	FReplicationServer::FReplicationServer(
		FAutomationTestBase& TestContext,
		EConcertSyncSessionFlags InSessionFlags,
		TSharedRef<ConcertSyncServer::Replication::IReplicationWorkspace> InWorkspace
		)
		: SessionFlags(InSessionFlags)
		, TestContext(TestContext)
		, ServerSessionMock(MakeShared<FConcertServerSessionMock>())
		, ReplicationWorkspace(MoveTemp(InWorkspace))
		, ServerReplicationManager(ConcertSyncServer::TestInterface::CreateServerReplicationManager(ServerSessionMock, *ReplicationWorkspace, InSessionFlags))
	{}

	FReplicationClient& FReplicationServer::ConnectClient(FConcertClientInfo ClientInfo)
	{
		const FGuid ClientEndpointId(0, 0, 0, Clients.Num() + 1); // {0, 0, 0, 0} is used by the server.
		Clients.Add(MakeUnique<FReplicationClient>(ClientEndpointId, SessionFlags, *ServerSessionMock, TestContext, MoveTemp(ClientInfo)));
		ServerSessionMock->ConnectClient(ClientEndpointId, *(Clients.Last()->GetClientSessionMock()));
		return *Clients.Last();
	}

	void FReplicationServer::TickServer(float FakeDeltaTime)
	{
		ServerSessionMock->OnTick().Broadcast(*ServerSessionMock, FakeDeltaTime);
	}
}
