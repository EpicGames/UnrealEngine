// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "ConcertSyncSessionFlags.h"
#include "Replication/IConcertServerReplicationManager.h"
#include "Replication/Util/Mocks/ReplicationWorkspaceEmptyMock.h"
#include "Util/ClientServerCommunicationTest.h"

#include "Templates/UnrealTemplate.h"

namespace UE::ConcertSyncServer::Replication { class IReplicationWorkspace; }

class FAutomationTestBase;

namespace UE::ConcertSyncTests::Replication
{
	class FReplicationClient;

	/** Reusable logic to simulate a test server in spec tests with. */
	class FReplicationServer : public FNoncopyable
	{
	public:

		FReplicationServer(
			FAutomationTestBase& TestContext,
			EConcertSyncSessionFlags InSessionFlags = EConcertSyncSessionFlags::Default_MultiUserSession,
			TSharedRef<ConcertSyncServer::Replication::IReplicationWorkspace> InWorkspace = MakeShared<FReplicationWorkspaceEmptyMock>()
			);

		/** Connects a client to the server. */
		FReplicationClient& ConnectClient(FConcertClientInfo ClientInfo = {});
		
		/** Lets the server process any messages that have come in. */
		void TickServer(float FakeDeltaTime = 1.f / 60.f);

		const TSharedRef<FConcertServerSessionMock>& GetServerSessionMock() const { return ServerSessionMock; }
	
	private:

		/** Relevant to some requests. Passed to ServerReplicationManager and clients upon creation. */
		const EConcertSyncSessionFlags SessionFlags;

		/** Used to test "obvious" cases that should never fail in any test. */
		FAutomationTestBase& TestContext;

		/** The underlying server session */
		TSharedRef<FConcertServerSessionMock> ServerSessionMock;
		/** Mock for the server workspace the replication system interacts with. */
		TSharedRef<ConcertSyncServer::Replication::IReplicationWorkspace> ReplicationWorkspace;
		/** Manages replication server side */
		TSharedRef<ConcertSyncServer::Replication::IConcertServerReplicationManager> ServerReplicationManager;

		/** Clients connected thus far. */
		TArray<TUniquePtr<FReplicationClient>> Clients;
	};
}


