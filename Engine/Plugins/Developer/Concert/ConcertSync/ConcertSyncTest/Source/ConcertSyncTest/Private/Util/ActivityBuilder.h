// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScopedSessionDatabaseWithEndpoint.h"

namespace UE::ConcertSyncTests
{
	/**
	 * Helps fill the database
	 */
	class FActivityBuilder
	{
		TArray<FActivityID> Activities;
		FScopedSessionDatabaseWithEndpoint& SessionDatabase;
	public:

		FActivityBuilder(FScopedSessionDatabaseWithEndpoint& SessionDatabase, uint32 ActivityCount)
			: SessionDatabase(SessionDatabase)
		{
			Activities.SetNumUninitialized(ActivityCount);
		}

		TArray<FActivityID> GetActivities() const { return Activities; }

		bool NewMap(FName MapName, uint32 ActivityIndex, EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None)
		{
			FConcertSyncActivity NewPackage;
			NewPackage.Flags = Flags;
			NewPackage.EndpointId = SessionDatabase.GetEndpoint();
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageInfo.PackageName = MapName;
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Added;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			int64 Dummy;
			return SessionDatabase.AddPackageActivity(NewPackage, PackageInfo, PackageDataStream, Activities[ActivityIndex], Dummy);
		}

		bool SaveMap(FName MapName, uint32 ActivityIndex, EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None)
		{
			FConcertSyncActivity SaveFooPackage;
			SaveFooPackage.Flags = Flags;
			SaveFooPackage.EndpointId = SessionDatabase.GetEndpoint();
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageInfo.PackageName = MapName;
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Saved;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			int64 Dummy;
			return SessionDatabase.AddPackageActivity(SaveFooPackage, PackageInfo, PackageDataStream, Activities[ActivityIndex], Dummy);
		}

		bool DeleteMap(FName MapName, uint32 ActivityIndex, EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None)
		{
			FConcertSyncActivity DeleteFooPackage;
			DeleteFooPackage.Flags = Flags;
			DeleteFooPackage.EndpointId = SessionDatabase.GetEndpoint();
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageInfo.PackageName = MapName;
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Deleted;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			int64 Dummy;
			return SessionDatabase.AddPackageActivity(DeleteFooPackage, PackageInfo, PackageDataStream, Activities[ActivityIndex], Dummy);
		}

		bool RenameMap(FName OldMapName, FName NewMapName, uint32 SaveActivityIndex, uint32 RenameActivityIndex, EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None)
		{
			int64 Dummy;
			
			FConcertSyncActivity SaveFoo2Package;
			SaveFoo2Package.Flags = Flags;
			SaveFoo2Package.EndpointId = SessionDatabase.GetEndpoint();
			FConcertPackageInfo PackageInfo;
			FConcertPackageDataStream PackageDataStream;
			PackageInfo.PackageName = NewMapName;
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Saved;
			SessionDatabase.GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
			bool bSuccess = SessionDatabase.AddPackageActivity(SaveFoo2Package, PackageInfo, PackageDataStream, Activities[SaveActivityIndex], Dummy);
			
			PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Renamed;
			PackageInfo.PackageName = OldMapName;
			PackageInfo.NewPackageName = NewMapName;
			bSuccess &= SessionDatabase.AddPackageActivity(SaveFoo2Package, PackageInfo, PackageDataStream, Activities[RenameActivityIndex], Dummy);

			return bSuccess;
		}

		bool CreateActor(FName MapName, uint32 ActivityIndex, FName ActorName = EName::Actor, EConcertSyncActivityFlags Flags = EConcertSyncActivityFlags::None)
		{
			FConcertExportedObject Actor;
			Actor.ObjectId.ObjectName = ActorName;
			Actor.ObjectId.ObjectPackageName = MapName;
			Actor.ObjectId.ObjectOuterPathName = *FString::Printf(TEXT("%s:PersistentLevel"), *MapName.ToString());
			Actor.ObjectId.ObjectClassPathName = TEXT("/Script/Engine.StaticMeshActor");
			
			FConcertSyncTransactionActivity CreateActorActivity;
			CreateActorActivity.Flags = Flags;
			CreateActorActivity.EndpointId = SessionDatabase.GetEndpoint();
			CreateActorActivity.EventData.Transaction.TransactionId = FGuid::NewGuid();
			CreateActorActivity.EventData.Transaction.OperationId = FGuid::NewGuid();
			Actor.ObjectData.bAllowCreate = true;
			CreateActorActivity.EventData.Transaction.ExportedObjects = { Actor };
			CreateActorActivity.EventData.Transaction.ModifiedPackages = { MapName };
			SessionDatabase.GetTransactionMaxEventId(CreateActorActivity.EventId);
			int64 Dummy;
			return SessionDatabase.AddTransactionActivity(CreateActorActivity, Activities[ActivityIndex], Dummy);
		}
	};
}
