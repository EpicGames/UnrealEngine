// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessages.h"
#include "IConcertSession.h"
#include "Mock/MockUtils.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "Scratchpad/ConcertScratchpad.h"
#include "Scratchpad/ConcertScratchpadPtr.h"
#include "UObject/StructOnScope.h"

namespace UE::ConcertSyncTests
{
	inline FString GetTestSessionRootPath()
	{
		return FPaths::ProjectIntermediateDir() / TEXT("ConcertDataStoreTest");
	}
	
	/** Implements a not-working IConcertServerSession. It must be further overridden to implement just what is required by the tests */
	class FConcertServerSessionBaseMock : public IConcertServerSession
	{
	public:
		FConcertServerSessionBaseMock() 
			: Id(FGuid::NewGuid())
			, Name("FConcertServerSessionBaseMock")
		{ }

		// IConcertSession Begin.
		virtual const FGuid& GetId() const override																								{ return Id; }
		virtual const FString& GetName() const override																							{ return NotMocked<const FString&>(Name); }
		virtual void SetName(const FString&) override																							{ return NotMocked<void>(); }
		virtual FMessageAddress GetClientAddress(const FGuid& ClientEndpointId) const override													{ return NotMocked<FMessageAddress>(); }
		virtual const FConcertSessionInfo& GetSessionInfo() const override																		{ return NotMocked<const FConcertSessionInfo&>(SessionInfo); }
		virtual FString GetSessionWorkingDirectory() const override																				{ return GetTestSessionRootPath() / Id.ToString(); }
		virtual TArray<FGuid> GetSessionClientEndpointIds() const override																		{ return NotMocked<TArray<FGuid>>(); }
		virtual TArray<FConcertSessionClientInfo> GetSessionClients() const override															{ return NotMocked<TArray<FConcertSessionClientInfo>>(); }
		virtual bool FindSessionClient(const FGuid&, FConcertSessionClientInfo&) const override													{ return NotMocked<bool>(); }
		virtual void Startup() override																											{ return NotMocked<void>(); }
		virtual void Shutdown() override																										{ return NotMocked<void>(); };
		virtual FConcertScratchpadRef GetScratchpad() const override																			{ return NotMocked<FConcertScratchpadRef>(); }
		virtual FConcertScratchpadPtr GetClientScratchpad(const FGuid&) const override															{ return NotMocked<FConcertScratchpadRef>(); }
		virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName&, const TSharedRef<IConcertSessionCustomEventHandler>&) override { return NotMocked<FDelegateHandle>(); }
		virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override			{ return NotMocked<void>(); }
		virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override						{ return NotMocked<void>(); }
		virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override													{ return NotMocked<void>(); }
		virtual void InternalSendCustomEvent(const UScriptStruct*, const void*, const TArray<FGuid>&, EConcertMessageFlags, TOptional<FConcertSequencedCustomEvent> InSequenceId={}) override			{ return NotMocked<void>(); }
		virtual void InternalRegisterCustomRequestHandler(const FName&, const TSharedRef<IConcertSessionCustomRequestHandler>&) override		{ return NotMocked<void>(); }
		virtual void InternalUnregisterCustomRequestHandler(const FName&) override																{ return NotMocked<void>(); }
		virtual void InternalSendCustomRequest(const UScriptStruct*, const void*, const FGuid&, const TSharedRef<IConcertSessionCustomResponseHandler>&) override { NotMocked<void>(); }
		// IConcertSession End.

		// IConcertServerSession Begin
		virtual FOnConcertServerSessionTick& OnTick() override                          { return NotMocked<FOnConcertServerSessionTick&>(Tick); }
		virtual FOnConcertServerSessionClientChanged& OnSessionClientChanged() override { return NotMocked<FOnConcertServerSessionClientChanged&>(ConnectionChanged); }
		virtual FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint& OnConcertMessageAcknowledgementReceived() override { return NotMocked<FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint&>(AckReceived); }
		// IConcertServerSession End

	protected:
		FGuid Id;

		// Those below are unused mocked data member, but required to get the code compiling.
		FString Name;
		FConcertSessionInfo SessionInfo;
		FOnConcertServerSessionTick Tick;
		FOnConcertServerSessionClientChanged ConnectionChanged;
		FOnConcertMessageAcknowledgementReceivedFromLocalEndpoint AckReceived;
	};

	/** Implements a not-working IConcertClientSession. It must be further overridden to implement just what is required by the tests */
	class FConcertClientSessionBaseMock : public IConcertClientSession
	{
	public:
		FConcertClientSessionBaseMock(FConcertClientInfo ClientInfo) 
			: Id(FGuid::NewGuid()) 
			, Name("FConcertClientSessionBaseMock")
			, ClientInfo(MoveTemp(ClientInfo))
		{ }

		// IConcertSession Begin.
		virtual const FGuid& GetId() const override																								{ return Id; }
		virtual const FString& GetName() const override																							{ return NotMocked<const FString&>(Name); }
		virtual const FConcertSessionInfo& GetSessionInfo() const override																		{ return NotMocked<const FConcertSessionInfo&>(SessionInfo); }
		virtual FString GetSessionWorkingDirectory() const override																				{ return GetTestSessionRootPath() / Id.ToString(); }
		virtual TArray<FGuid> GetSessionClientEndpointIds() const override																		{ return NotMocked<TArray<FGuid>>(); }
		virtual TArray<FConcertSessionClientInfo> GetSessionClients() const override															{ return NotMocked<TArray<FConcertSessionClientInfo>>(); }
		virtual bool FindSessionClient(const FGuid&, FConcertSessionClientInfo&) const override													{ return NotMocked<bool>(); }
		virtual void Startup() override																											{ return NotMocked<void>(); }
		virtual void Shutdown() override																										{ return NotMocked<void>(); };
		virtual FConcertScratchpadRef GetScratchpad() const override																			{ return NotMocked<FConcertScratchpadRef>(); }
		virtual FConcertScratchpadPtr GetClientScratchpad(const FGuid&) const override															{ return NotMocked<FConcertScratchpadRef>(); }
		virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName&, const TSharedRef<IConcertSessionCustomEventHandler>&) override { return NotMocked<FDelegateHandle>(); }
		virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override			{ return NotMocked<void>(); }
		virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override						{ return NotMocked<void>(); }
		virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override													{ return NotMocked<void>(); }
		virtual void InternalSendCustomEvent(const UScriptStruct*, const void*, const TArray<FGuid>&, EConcertMessageFlags, TOptional<FConcertSequencedCustomEvent> InSequenceId={}) override			{ return NotMocked<void>(); }
		virtual void InternalRegisterCustomRequestHandler(const FName&, const TSharedRef<IConcertSessionCustomRequestHandler>&) override		{ return NotMocked<void>(); }
		virtual void InternalUnregisterCustomRequestHandler(const FName&) override																{ return NotMocked<void>(); }
		virtual void InternalSendCustomRequest(const UScriptStruct*, const void*, const FGuid&, const TSharedRef<IConcertSessionCustomResponseHandler>&) override { NotMocked<void>(); }
		// IConcertSession End.

		virtual FConcertSequencedCustomEventManager& GetSequencedEventManager() override
		{
			return NotMocked<FConcertSequencedCustomEventManager&>(CustomEventSequenceManager);
		}
		// IConcertClientSession Begin
		virtual EConcertConnectionStatus GetConnectionStatus() const override            { return NotMocked<EConcertConnectionStatus>(EConcertConnectionStatus::Connected); }
		virtual FGuid GetSessionClientEndpointId() const override                        { return NotMocked<FGuid>(); }
		virtual FGuid GetSessionServerEndpointId() const override                        { return NotMocked<FGuid>(); }
		virtual const FConcertClientInfo& GetLocalClientInfo() const override            { return ClientInfo; }
		virtual void UpdateLocalClientInfo(const FConcertClientInfoUpdate&) override     { return NotMocked<void>(); }
		virtual void Connect() override                                                  { return NotMocked<void>(); }
		virtual void Disconnect() override                                               { return NotMocked<void>(); }

		virtual EConcertSendReceiveState GetSendReceiveState() const override
		{
			return NotMocked<EConcertSendReceiveState>(EConcertSendReceiveState::Default);
		}

		virtual void SetSendReceiveState(EConcertSendReceiveState InSendReceiveState) override
		{
			return NotMocked<void>();
		}

		virtual FOnConcertClientSessionTick& OnTick() override                           { return NotMocked<FOnConcertClientSessionTick&>(Tick); }
		virtual FOnConcertClientSessionConnectionChanged& OnConnectionChanged() override { return NotMocked<FOnConcertClientSessionConnectionChanged&>(ConnectionChanged); }
		virtual FOnConcertClientSessionClientChanged& OnSessionClientChanged() override  { return NotMocked<FOnConcertClientSessionClientChanged&>(ClientChanged); }
		virtual FOnConcertSessionRenamed& OnSessionRenamed() override                    { return NotMocked<FOnConcertSessionRenamed&>(SessionRenamed); }
		// IConcertClientSession End

		virtual void HandleCustomEvent(const UScriptStruct* EventType, const void* EventData) = 0;

	protected:
		FGuid Id;

		// Those below are unused mocked data member, but required to get the code compiling.
		FString Name;
		FConcertSessionInfo SessionInfo;
		FOnConcertClientSessionTick Tick;
		FOnConcertClientSessionConnectionChanged ConnectionChanged;
		FOnConcertClientSessionClientChanged ClientChanged;
		FOnConcertSessionRenamed SessionRenamed;
		FConcertSequencedCustomEventManager CustomEventSequenceManager;
		FConcertClientInfo ClientInfo;
	};

	enum class EServerSessionTestingFlags
	{
		None = 0,
		/** In FConcertServerSessionMock::DispatchEvent if no request handler is registered, it is allowed to timeout the request. Otherwise a check() is triggered. */
		AllowRequestTimeouts = 1 << 0
	};
	ENUM_CLASS_FLAGS(EServerSessionTestingFlags);

	/** Specializes the base concert server session to act as a fake server session. */
	class FConcertServerSessionMock : public FConcertServerSessionBaseMock
	{
	public:

		FConcertServerSessionMock()
		{
			RegisterCustomEventHandler<FConcertSession_LeaveSessionEvent>(this, &FConcertServerSessionMock::HandleClientLeaveEvent);
		}
		
		void SetTestFlags(EServerSessionTestingFlags InFlags)
		{
			TestFlags = InFlags;
		}
		
		virtual void InternalSendCustomEvent(const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& TargetEndpointIds, EConcertMessageFlags, TOptional<FConcertSequencedCustomEvent> InSequenceId={}) override
		{
			for (const FGuid& TargetEndPointId : TargetEndpointIds)
			{
				int i = 0;
				for (const FGuid& ClientEndPointId : ClientEndpoints)
				{
					if (TargetEndPointId == ClientEndPointId)
					{
						// Dispatch the event on the client immediately.
						ClientSessions[i]->HandleCustomEvent(EventType, EventData);
					}
					++i;
				}
			}
		}

		virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName& EventMessageType, const TSharedRef<IConcertSessionCustomEventHandler>& Handler) override
		{
			CustomEventHandlers.FindOrAdd(EventMessageType).Add(Handler);
			return Handler->GetHandle();
		}

		virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override
		{
			TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventMessageType);
			if (HandlerArrayPtr)
			{
				for (auto It = HandlerArrayPtr->CreateIterator(); It; ++It)
				{
					if ((*It)->GetHandle() == EventHandle)
					{
						It.RemoveCurrent();
						break;
					}
				}
			}
		}

		virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override
		{
			TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventMessageType);
			if (HandlerArrayPtr)
			{
				for (auto It = HandlerArrayPtr->CreateIterator(); It; ++It)
				{
					if ((*It)->HasSameObject(EventHandler))
					{
						It.RemoveCurrent();
						break;
					}
				}
			}
		}

		virtual void InternalRegisterCustomRequestHandler(const FName& RequestMessageType, const TSharedRef<IConcertSessionCustomRequestHandler>& Handler) override
		{
			CustomRequestHandlers.Add(RequestMessageType, Handler);
		}

		virtual void InternalUnregisterCustomRequestHandler(const FName& RequestMessageType) override
		{
			CustomRequestHandlers.Remove(RequestMessageType);
		}

		virtual TArray<FGuid> GetSessionClientEndpointIds() const override
		{
			return ClientEndpoints;
		}

		// Called by the tests to connect a client to the server.
		void ConnectClient(const FGuid& ClientEndpointId, FConcertClientSessionBaseMock& ClientSession)
		{
			// Notify the server that a new client connected. The server data store will replicate its content on the client.
			FConcertSessionClientInfo Info;
			Info.ClientEndpointId = ClientEndpointId;
			ClientEndpoints.Add(ClientEndpointId);
			ClientSessions.Add(&ClientSession);
			ConnectionChanged.Broadcast(*this, EConcertClientStatus::Connected, Info);
		}

		void DispatchEvent(const FGuid& RequesterEndpointId, const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& DestinationEndpointIds, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> InSequenceId)
		{
			// { 0, 0, 0, 0 } is for the server
			if (DestinationEndpointIds.Contains(FGuid{ 0, 0, 0, 0 }))
			{
				if (const TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* Handlers = CustomEventHandlers.Find(EventType->GetFName()))
				{
					FConcertSessionContext DummyContext;
					DummyContext.SourceEndpointId = RequesterEndpointId;
					for (const TSharedPtr<IConcertSessionCustomEventHandler>& Handler : *Handlers)
					{
						Handler->HandleEvent(DummyContext, EventData);
					}
				}
			}
			
			InternalSendCustomEvent(EventType, EventData, DestinationEndpointIds, Flags, InSequenceId);
		}

		// Called by the FConcertClientSessionMock to dispatch a request
		void DispatchRequest(const FGuid& RequesterEndpointId, const UScriptStruct* RequestType, const void* RequestData, const TSharedRef<IConcertSessionCustomResponseHandler>& ResponseHandler)
		{
			if (TSharedPtr<IConcertSessionCustomRequestHandler>* RequestHandler = CustomRequestHandlers.Find(RequestType->GetFName()))
			{
				// Set up who's sending the request.
				FConcertSessionContext Context;
				Context.SourceEndpointId = RequesterEndpointId;

				// Dispatch the request
				FStructOnScope ResponsePayload((*RequestHandler)->GetResponseType());
				EConcertSessionResponseCode Result = (*RequestHandler)->HandleRequest(Context, RequestData, ResponsePayload.GetStructMemory());
				if (Result == EConcertSessionResponseCode::Success || Result == EConcertSessionResponseCode::Failed)
				{
					// Dispatch the response.
					ResponseHandler->HandleResponse(ResponsePayload.GetStructMemory());
				}
			}
			else 
			{
				const bool bAllowTimeout = EnumHasAnyFlags(TestFlags, EServerSessionTestingFlags::AllowRequestTimeouts);
				if (ensureAlways(bAllowTimeout))
				{
					ResponseHandler->HandleResponse(nullptr);
				}
			}
		}

		virtual bool FindSessionClient(const FGuid& EndpointId, FConcertSessionClientInfo& OutSessionClientInfo) const override
		{
			for (int32 i = 0; i < ClientSessions.Num(); ++i)
			{
				if (ClientEndpoints[i] != EndpointId)
				{
					continue;
				}
				
				const FConcertClientSessionBaseMock* ClientSessionMock = ClientSessions[i];
				OutSessionClientInfo = { EndpointId, ClientSessionMock->GetLocalClientInfo() };
				return true;
			}
			return false;
		}
		
		virtual FOnConcertServerSessionClientChanged& OnSessionClientChanged() override
		{
			return ConnectionChanged;
		}

		virtual FOnConcertServerSessionTick& OnTick() override { return OnTickDelegate; }

	private:

		/** Tick delegate for tests that need it */
		FOnConcertServerSessionTick OnTickDelegate;
		
		/** Map of session custom event handlers */
		TMap<FName, TArray<TSharedPtr<IConcertSessionCustomEventHandler>>> CustomEventHandlers;
		/** Map of session custom request handlers */
		TMap<FName, TSharedPtr<IConcertSessionCustomRequestHandler>> CustomRequestHandlers;

		/** Connected client endpoints. */
		TArray<FGuid> ClientEndpoints;

		/** Connected clients sessions. */
		TArray<FConcertClientSessionBaseMock*> ClientSessions;

		EServerSessionTestingFlags TestFlags = EServerSessionTestingFlags::None;
		
		void HandleClientLeaveEvent(const FConcertSessionContext& Context, const FConcertSession_LeaveSessionEvent& Event)
		{
			const FGuid& ClientEndpointId = Context.SourceEndpointId;
			ClientEndpoints.RemoveSingle(ClientEndpointId);
			
			const int32 Index = ClientSessions.IndexOfByPredicate([&ClientEndpointId](FConcertClientSessionBaseMock* SessionMock){ return SessionMock->GetSessionClientEndpointId() == ClientEndpointId; });
			check(ClientSessions.IsValidIndex(Index));
			
			ConnectionChanged.Broadcast(*this, EConcertClientStatus::Disconnected, { ClientEndpointId });
		}
	};

	/** Specializes the base concert client session to act as a fake client session. */
	class FConcertClientSessionMock : public FConcertClientSessionBaseMock
	{
	public:
		
		FConcertClientSessionMock(const FGuid& ClientEndpointId, FConcertServerSessionMock& Server, FConcertClientInfo ClientInfo)
			: FConcertClientSessionBaseMock(MoveTemp(ClientInfo))
			, ServerMock(Server)
			, EndpointId(ClientEndpointId)
		{
		}

		virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName& EventMessageType, const TSharedRef<IConcertSessionCustomEventHandler>& Handler) override
		{
			CustomEventHandlers.FindOrAdd(EventMessageType).Add(Handler);
			return Handler->GetHandle();
		}

		virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override
		{
			TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventMessageType);
			if (HandlerArrayPtr)
			{
				for (auto It = HandlerArrayPtr->CreateIterator(); It; ++It)
				{
					if ((*It)->GetHandle() == EventHandle)
					{
						It.RemoveCurrent();
						break;
					}
				}
			}
		}

		virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override
		{
			TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventMessageType);
			if (HandlerArrayPtr)
			{
				HandlerArrayPtr->RemoveAllSwap([EventHandler](const TSharedPtr<IConcertSessionCustomEventHandler>& InHandler) -> bool
				{
					return InHandler->HasSameObject(EventHandler);
				});
			}
		}

		virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override
		{
			CustomEventHandlers.Remove(EventMessageType);
		}

		virtual void InternalSendCustomEvent(const UScriptStruct* EventType, const void* EventData, const TArray<FGuid>& DestinationEndpointIds, EConcertMessageFlags Flags, TOptional<FConcertSequencedCustomEvent> InSequenceId) override
		{
			ServerMock.DispatchEvent(EndpointId, EventType, EventData, DestinationEndpointIds, Flags, InSequenceId);
		}

		virtual void InternalSendCustomRequest(const UScriptStruct* RequestType, const void* RequestData, const FGuid& DestinationEndpointId, const TSharedRef<IConcertSessionCustomResponseHandler>& Handler) override
		{
			// Directly dispatch to the server session.
			ServerMock.DispatchRequest(EndpointId, RequestType, RequestData, Handler);
		}

		virtual void HandleCustomEvent(const UScriptStruct* EventType, const void* EventData) override
		{
			TArray<TSharedPtr<IConcertSessionCustomEventHandler>>* HandlerArrayPtr = CustomEventHandlers.Find(EventType->GetFName());
			if (HandlerArrayPtr)
			{
				FConcertSessionContext DummyContext;
				for (const TSharedPtr<IConcertSessionCustomEventHandler>& Handler : *HandlerArrayPtr)
				{
					Handler->HandleEvent(DummyContext, EventData);
				}
			}
		}

		virtual FGuid GetSessionClientEndpointId() const override
		{
			return EndpointId;
		}

		virtual FGuid GetSessionServerEndpointId() const override
		{
			return FGuid(0, 0, 0, 0);
		}

		virtual void Disconnect() override
		{
			if (ensure(bIsConnected))
			{
				bIsConnected = false;
				FConcertSession_LeaveSessionEvent LeaveSessionEvent;
				LeaveSessionEvent.SessionServerEndpointId = SessionInfo.ServerEndpointId;
				ServerMock.DispatchEvent(EndpointId, FConcertSession_LeaveSessionEvent::StaticStruct(), &LeaveSessionEvent, { SessionInfo.ServerEndpointId }, EConcertMessageFlags::ReliableOrdered, {});
			}
		}
		
		virtual FOnConcertClientSessionTick& OnTick() override { return OnTickDelegate; }

	private:
		FOnConcertClientSessionTick OnTickDelegate;
		FConcertServerSessionMock& ServerMock;
		FGuid EndpointId;
		bool bIsConnected = true;
		TMap<FName, TArray<TSharedPtr<IConcertSessionCustomEventHandler>>> CustomEventHandlers;
	};

	/** Base class to perform Concert data store client/server tests. */
	class FConcertClientServerCommunicationTest : public FAutomationTestBase
	{
	public:
		struct FClientInfo
		{
			FClientInfo(const FGuid& ClientEndPointId, FConcertServerSessionMock& Server, FConcertClientInfo ClientInfo)
				: ClientSessionMock(MakeShared<FConcertClientSessionMock>(ClientEndPointId, Server, MoveTemp(ClientInfo)))
			{}

			const TSharedRef<FConcertClientSessionBaseMock> ClientSessionMock;
		};

		FConcertClientServerCommunicationTest(const FString& InName, const bool bInComplexTask)
			: FAutomationTestBase(InName, bInComplexTask)
		{
			FAutomationTestFramework& Framework = FAutomationTestFramework::Get();
			Framework.OnTestEndEvent.AddRaw(this, &FConcertClientServerCommunicationTest::CleanUpTest);
		}
		
		virtual ~FConcertClientServerCommunicationTest() override
		{
			FAutomationTestFramework& Framework = FAutomationTestFramework::Get();
			Framework.OnTestEndEvent.RemoveAll(this);
		}

		FClientInfo& ConnectClient(FConcertClientInfo ClientInfo = {})
		{
			const FGuid ClientEndpointId(0, 0, 0, Clients.Num() + 1); // {0, 0, 0, 0} is used by the server.
			Clients.Add(MakeUnique<FClientInfo>(ClientEndpointId, *ServerSessionMock, MoveTemp(ClientInfo)));
			ServerSessionMock->ConnectClient(ClientEndpointId, *(Clients.Last()->ClientSessionMock));
			return *Clients.Last();
		}

		void InitServer()
		{
			// Reset everything to be able to rerun the tests. The test framework doesn't destruct/reconstruct this object
			// at every run, so just ensure we start with a clean state.
			Clients.Empty();
			ServerSessionMock = MakeShared<FConcertServerSessionMock>();
		}

		const TSharedPtr<FConcertServerSessionMock>& GetServerSessionMock() const { return ServerSessionMock; }

		virtual void CleanUpTest(FAutomationTestBase* AutomationTestBase)
		{
			if (AutomationTestBase == this)
			{
				Clients.Reset();
				ServerSessionMock.Reset();
			}
		}
		
	private:
		TSharedPtr<FConcertServerSessionMock> ServerSessionMock;
		TArray<TUniquePtr<FClientInfo>> Clients;
	};
}