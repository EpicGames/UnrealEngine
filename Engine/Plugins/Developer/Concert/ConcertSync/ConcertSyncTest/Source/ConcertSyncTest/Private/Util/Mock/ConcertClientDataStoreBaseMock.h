// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertClientDataStore.h"
#include "MockUtils.h"

namespace UE::ConcertSyncTests::Replication
{
	/** Implements every function with NotMocked. */
	class FConcertClientDataStoreBaseMock : public IConcertClientDataStore
	{
	protected:
		
		//~ Begin IConcertClientWorkspace Interface
		virtual TFuture<FConcertDataStoreResult> InternalFetchOrAdd(const FName& Key, const UScriptStruct* Type, const FName& TypeName, const void* Payload) override { return NotMocked<TFuture<FConcertDataStoreResult>>(); }
		virtual TFuture<FConcertDataStoreResult> InternalFetchAs(const FName& Key, const UScriptStruct* Type, const FName& TypeName) const override { return NotMocked<TFuture<FConcertDataStoreResult>>(); }
		virtual TFuture<FConcertDataStoreResult> InternalCompareExchange(const FName& Key, const UScriptStruct* Type, const FName& TypeName, const void* Expected, const void* Desired) override { return NotMocked<TFuture<FConcertDataStoreResult>>(); }
		virtual void InternalRegisterChangeNotificationHandler(const FName& Key, const FName& TypeName, const FChangeNotificationHandler& Handler, EConcertDataStoreChangeNotificationOptions Options) override { NotMocked<void>(); }
		virtual void InternalUnregisterChangeNotificationHandler(const FName& Key) override { NotMocked<void>(); }
		//~ End IConcertClientWorkspace Interface
	};
}
