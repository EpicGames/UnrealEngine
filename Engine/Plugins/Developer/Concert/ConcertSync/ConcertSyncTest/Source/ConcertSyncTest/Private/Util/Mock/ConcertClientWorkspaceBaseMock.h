// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertClientDataStoreBaseMock.h"
#include "IConcertClientWorkspace.h"
#include "MockUtils.h"
#include "Util/ClientServerCommunicationTest.h"

namespace UE::ConcertSyncTests::Replication
{
	/** Implements every function with NotMocked. */
	class FConcertClientWorkspaceBaseMock : public IConcertClientWorkspace
	{
	public:

		//~ Begin IConcertClientWorkspace Interface
		virtual IConcertClientSession& GetSession() const override { return NotMocked<IConcertClientSession&>(*SessionMock); }
		virtual FGuid GetWorkspaceLockId() const override { return NotMocked<FGuid>(); }
		virtual FGuid GetResourceLockId(const FName InResourceName) const override { return NotMocked<FGuid>(); }
		virtual bool AreResourcesLockedBy(TArrayView<const FName> ResourceNames, const FGuid& ClientId) override { return NotMocked<bool>(); }
		virtual TFuture<FConcertResourceLockResponse> LockResources(TArray<FName> InResourceName) override { return NotMocked<TFuture<FConcertResourceLockResponse>>(); }
		virtual TFuture<FConcertResourceLockResponse> UnlockResources(TArray<FName> InResourceName) override { return NotMocked<TFuture<FConcertResourceLockResponse>>(); }
		virtual bool HasSessionChanges() const override { return NotMocked<bool>(); }
		virtual TArray<FName> GatherSessionChanges(bool IgnorePersisted) override { return NotMocked<TArray<FName>>(); }
		virtual TOptional<FString> GetValidPackageSessionPath(FName PackageName) const override { return NotMocked<TOptional<FString>>(); }
		virtual FPersistResult PersistSessionChanges(FPersistParameters InParam) override { return NotMocked<FPersistResult>(); }
		virtual void GetActivities(const int64 FirstActivityIdToFetch, const int64 MaxNumActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, TArray<FConcertSessionActivity>& OutActivities) const override { NotMocked<void>(); }
		virtual int64 GetLastActivityId() const override { return NotMocked<int64>(); }
		virtual FOnActivityAddedOrUpdated& OnActivityAddedOrUpdated() override { return NotMocked<FOnActivityAddedOrUpdated&>(OnActivityAddedOrUpdatedDelegate); }
		virtual bool HasLiveTransactionSupport(UPackage* InPackage) const override { return NotMocked<bool>(); }
		virtual bool ShouldIgnorePackageDirtyEvent(UPackage* InPackage) const override { return NotMocked<bool>(); }
		virtual bool FindTransactionEvent(const int64 TransactionEventId, FConcertSyncTransactionEvent& OutTransactionEvent, const bool bMetaDataOnly) const override { return NotMocked<bool>(); }
		virtual TFuture<TOptional<FConcertSyncTransactionEvent>> FindOrRequestTransactionEvent(const int64 TransactionEventId, const bool bMetaDataOnly) override { return NotMocked<TFuture<TOptional<FConcertSyncTransactionEvent>>>(); }
		virtual bool FindPackageEvent(const int64 PackageEventId, FConcertSyncPackageEventMetaData& OutPackageEvent) const override { return NotMocked<bool>(); }
		virtual bool FindReplicationEvent(const int64 ReplicationEventId, FConcertSyncReplicationEvent& OutReplicationEvent) const override { return NotMocked<bool>(); }
		virtual FOnWorkspaceSynchronized& OnWorkspaceSynchronized() override { return NotMocked<FOnWorkspaceSynchronized&>(OnWorkspaceSynchronizedDelegate); }
		virtual FOnFinalizeWorkspaceSyncCompleted& OnFinalizeWorkspaceSyncCompleted() override { return NotMocked<FOnFinalizeWorkspaceSyncCompleted&>(OnFinalizeWorkspaceSyncCompletedDelegate); }
		virtual void AddWorkspaceFinalizeDelegate(FName InDelegateName, FCanFinalizeWorkspaceDelegate InDelegate) override { NotMocked<void>(); }
		virtual void RemoveWorkspaceFinalizeDelegate(FName InDelegateName) override { NotMocked<void>(); }
		virtual void AddWorkspaceCanProcessPackagesDelegate(FName InDelegateName, FCanProcessPendingPackages Delegate) override { NotMocked<void>(); }
		virtual void RemoveWorkspaceCanProcessPackagesDelegate(FName InDelegateName) override { NotMocked<void>(); }
		virtual bool IsReloadingPackage(FName PackageName) const override { return NotMocked<bool>(); } 
		virtual IConcertClientDataStore& GetDataStore() override { return NotMocked<IConcertClientDataStore&>(DataStore); }
		virtual bool IsAssetModifiedByOtherClients(const FName& AssetName, int32* OutOtherClientsWithModifNum, TArray<FConcertClientInfo>* OutOtherClientsWithModifInfo, int32 OtherClientsWithModifMaxFetchNum) const override { return NotMocked<bool>(); }
		virtual void SetIgnoreOnRestoreFlagForEmittedActivities(bool bIgnore) override { NotMocked<void>(); }
		//~ End IConcertClientWorkspace Interface

		virtual ~FConcertClientWorkspaceBaseMock() = default;

	private:
		
		// Required to get stuff to compile
		class FSessionMock : public FConcertClientSessionBaseMock
		{
		public:
			explicit FSessionMock() : FConcertClientSessionBaseMock(FConcertClientInfo{}){}
			virtual void HandleCustomEvent(const UScriptStruct* EventType, const void* EventData) override {}
		};
		TSharedRef<FSessionMock> SessionMock = MakeShared<FSessionMock>();
		FOnActivityAddedOrUpdated OnActivityAddedOrUpdatedDelegate;
		FOnWorkspaceSynchronized OnWorkspaceSynchronizedDelegate;
		FOnFinalizeWorkspaceSyncCompleted OnFinalizeWorkspaceSyncCompletedDelegate;
		FConcertClientDataStoreBaseMock DataStore;
	};
}
