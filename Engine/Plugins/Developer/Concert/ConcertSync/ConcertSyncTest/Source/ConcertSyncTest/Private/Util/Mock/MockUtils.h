// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"

namespace UE::ConcertSyncTests
{
	// Utility functions used to detect when a non-mocked function is called, so that we can mock it properly when required.
	template<typename T> T NotMocked(T Ret) { check(false); return Ret; }
	template<typename T> T NotMocked()      { check(false); return T(); }
}