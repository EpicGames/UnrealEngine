// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncSessionDatabase.h"

#include "ConcertSyncSessionTypes.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"

namespace UE::ConcertSyncTests
{
	class FScopedSessionDatabase : public FConcertSyncSessionDatabase
	{
		FAutomationTestBase& Test;
		const FString TestSessionPath_Server = FPaths::ProjectIntermediateDir() / TEXT("ConcertDatabaseTest_Server");
	public:

		FScopedSessionDatabase(FAutomationTestBase& Test)
			: Test(Test)
		{
			Open(TestSessionPath_Server);
		}

		~FScopedSessionDatabase()
		{
			if (IsValid() && !Close())
			{
				Test.AddError(TEXT("Failed to close server database"));
			}
			IFileManager::Get().DeleteDirectory(*TestSessionPath_Server, false, true);
		}
	};
}
