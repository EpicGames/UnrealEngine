// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScopedSessionDatabase.h"

namespace UE::ConcertSyncTests
{
	class FScopedSessionDatabaseWithEndpoint : public FScopedSessionDatabase
	{
		const FGuid EndpointID;
	public:

		FScopedSessionDatabaseWithEndpoint(FAutomationTestBase& Test)
			: FScopedSessionDatabase(Test)
			, EndpointID(FGuid::NewGuid())
		{
			FConcertSyncEndpointData EndpointData;
			EndpointData.ClientInfo.Initialize();
			if (!SetEndpoint(EndpointID, EndpointData))
			{
				Test.AddError(FString::Printf(TEXT("Test may be faulty because endpoint could not be set: %s"), *GetLastError()));
			}
		}

		const FGuid& GetEndpoint() const { return EndpointID; }
	};
}
